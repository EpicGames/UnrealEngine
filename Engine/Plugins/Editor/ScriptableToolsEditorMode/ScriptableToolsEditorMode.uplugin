{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Scriptable Tools Editor Mode",
	"Description": "Editor Mode for Scriptable Tools",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
    "EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ScriptableToolsEditorMode",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ScriptableToolsFramework",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		}	
	]
}
