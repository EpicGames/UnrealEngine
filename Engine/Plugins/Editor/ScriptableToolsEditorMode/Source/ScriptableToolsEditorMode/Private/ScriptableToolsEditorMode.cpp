// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolsEditorMode.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "Modules/ModuleManager.h"
#include "ILevelEditor.h"
#include "LevelEditor.h"
#include "InteractiveTool.h"
#include "SLevelViewport.h"
#include "Application/ThrottleManager.h"

#include "InteractiveToolManager.h"
#include "ScriptableToolsEditorModeToolkit.h"
#include "ScriptableToolsEditorModeManagerCommands.h"
#include "ScriptableToolsEditorModeSettings.h"
#include "ScriptableToolsEditorModeStyle.h"

#include "BaseGizmos/TransformGizmoUtil.h"
#include "Snapping/ModelingSceneSnappingManager.h"

#include "ScriptableToolBuilder.h"
#include "ScriptableToolSet.h"
#include "ScriptableInteractiveTool.h"

#include "InteractiveToolQueryInterfaces.h" // IInteractiveToolExclusiveToolAPI
#include "ToolContextInterfaces.h"

#include "ToolTargetManager.h"
#include "ToolTargets/StaticMeshComponentToolTarget.h"
#include "ToolTargets/VolumeComponentToolTarget.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"
#include "ToolTargets/SkeletalMeshComponentToolTarget.h"

#include "Utility/ScriptableToolContextObjects.h"
#include "ContextObjectStore.h"

#include "Engine/StreamableManager.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Slate/SceneViewport.h"

#if WITH_UNREAL_DEVELOPER_TOOLS
#include "Overlay/SDraggableBoxOverlay.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolsEditorMode)



#define LOCTEXT_NAMESPACE "UScriptableToolsEditorMode"

const FEditorModeID UScriptableToolsEditorMode::EM_ScriptableToolsEditorModeId = TEXT("EM_ScriptableToolsEditorMode");

UScriptableToolsEditorMode::UScriptableToolsEditorMode()
{
	Info = FEditorModeInfo(
		EM_ScriptableToolsEditorModeId,
		LOCTEXT("ScriptableToolsEditorModeName", "Scriptable Tools"),
		FSlateIcon("ScriptableToolsEditorModeStyle", "LevelEditor.ScriptableToolsEditorMode", "LevelEditor.ScriptableToolsEditorMode.Small"),
		true,
		999999);
}

UScriptableToolsEditorMode::UScriptableToolsEditorMode(FVTableHelper& Helper)
	: UBaseLegacyWidgetEdMode(Helper)
{
}

UScriptableToolsEditorMode::~UScriptableToolsEditorMode()
{
}

bool UScriptableToolsEditorMode::ProcessEditDelete()
{
	if (UEdMode::ProcessEditDelete())
	{
		return true;
	}

	// for now we disable deleting in an Accept-style tool because it can result in crashes if we are deleting target object
	if ( GetToolManager()->HasAnyActiveTool() && GetToolManager()->GetActiveTool(EToolSide::Mouse)->HasAccept() )
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("CannotDeleteWarning", "Cannot delete objects while this Tool is active"), EToolMessageLevel::UserWarning);
		return true;
	}

	return false;
}


bool UScriptableToolsEditorMode::ProcessEditCut()
{
	// for now we disable deleting in an Accept-style tool because it can result in crashes if we are deleting target object
	if (GetToolManager()->HasAnyActiveTool() && GetToolManager()->GetActiveTool(EToolSide::Mouse)->HasAccept())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("CannotCutWarning", "Cannot cut objects while this Tool is active"), EToolMessageLevel::UserWarning);
		return true;
	}

	return false;
}


void UScriptableToolsEditorMode::ActorSelectionChangeNotify()
{
	// would like to clear selection here, but this is called multiple times, including after a transaction when
	// we cannot identify that the selection should not be cleared
}


bool UScriptableToolsEditorMode::CanAutoSave() const
{
	// prevent autosave if any tool is active
	return GetToolManager()->HasAnyActiveTool() == false;
}

bool UScriptableToolsEditorMode::ShouldDrawWidget() const
{ 
	// hide standard xform gizmo if we have an active tool
	if (GetInteractiveToolsContext() != nullptr && GetToolManager()->HasAnyActiveTool())
	{
		return false;
	}

	return UBaseLegacyWidgetEdMode::ShouldDrawWidget(); 
}

void UScriptableToolsEditorMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	Super::Tick(ViewportClient, DeltaTime);

	if (Toolkit.IsValid() && FApp::HasFocus())
	{
		FScriptableToolsEditorModeToolkit* ModeToolkit = (FScriptableToolsEditorModeToolkit*)Toolkit.Get();
		ModeToolkit->EnableShowRealtimeWarning(ViewportClient->IsRealtime() == false);
	}

	if (bRebuildScriptableToolSetOnTick)
	{
		RebuildScriptableToolSet();
		bRebuildScriptableToolSetOnTick = false;
	}
}


void UScriptableToolsEditorMode::Enter()
{
	UEdMode::Enter();

	// listen to post-build
	GetToolManager()->OnToolPostBuild.AddUObject(this, &UScriptableToolsEditorMode::OnToolPostBuild);

	// Register builders for tool targets that the mode uses.
	// TODO: We're not actually suporting modeling mode tool targets on scriptable tools, but the infrastructure to test for selected
	// objects uses the ToolTargetFactories, so we're including these here. We probably need a more generic way to accomplish this.
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UStaticMeshComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UVolumeComponentToolTargetFactory>(GetToolManager()));
	GetInteractiveToolsContext()->TargetManager->AddTargetFactory(NewObject<UDynamicMeshComponentToolTargetFactory>(GetToolManager()));

	//// forward shutdown requests
	//GetToolManager()->OnToolShutdownRequest.BindLambda([this](UInteractiveToolManager*, UInteractiveTool* Tool, EToolShutdownType ShutdownType)
	//{
	//	GetInteractiveToolsContext()->EndTool(ShutdownType); 
	//	return true;
	//});

	// register gizmo helper
	UE::TransformGizmoUtil::RegisterTransformGizmoContextObject(GetInteractiveToolsContext());

	// register snapping manager
	UE::Geometry::RegisterSceneSnappingManager(GetInteractiveToolsContext());
	//SceneSnappingManager = UE::Geometry::FindModelingSceneSnappingManager(GetToolManager());

	const FScriptableToolsEditorModeManagerCommands& ModeToolCommands = FScriptableToolsEditorModeManagerCommands::Get();

	// enable realtime viewport override
	ConfigureRealTimeViewportsOverride(true);

	// Ensure the ModeManager has initialized a focused viewport client otherwise the tool manager will not Tick.
	FEditorModeTools& ModeManager = GLevelEditorModeTools();
	if (ModeManager.GetFocusedViewportClient() == nullptr && GEditor)
	{
		if (FViewport* ActiveViewport = GEditor->GetActiveViewport())
		{
			ModeManager.ReceivedFocus(static_cast<FEditorViewportClient*>(ActiveViewport->GetClient()), ActiveViewport);
		}
	}

	ScriptableTools = NewObject<UScriptableToolSet>(this);

	UScriptableToolsModeCustomizationSettings* ModeSettings = GetMutableDefault<UScriptableToolsModeCustomizationSettings>();
	ModeSettings->OnSettingChanged().AddWeakLambda(this, [this](UObject*, FPropertyChangedEvent&)
	{
		RebuildScriptableToolSet();
	});

	// todoz
	GetToolManager()->SelectActiveToolType(EToolSide::Left, TEXT("BeginMeshInspectorTool"));

	BlueprintPreCompileHandle = GEditor->OnBlueprintPreCompile().AddUObject(this, &UScriptableToolsEditorMode::OnBlueprintPreCompile); 

	// do any toolkit UI initialization that depends on the mode setup above
	if (Toolkit.IsValid())
	{
		FScriptableToolsEditorModeToolkit* ModeToolkit = (FScriptableToolsEditorModeToolkit*)Toolkit.Get();
		ModeToolkit->InitializeAfterModeSetup();
	}

	RebuildScriptableToolSet();

	InitializeModeContexts();
}

void UScriptableToolsEditorMode::RebuildScriptableToolSet()
{
	auto UnregisterTools = [this]()
	{
		// unregister old tools from ToolManager
		ScriptableTools->ForEachScriptableTool([this](UClass* ToolClass, UInteractiveToolBuilder* ToolBuilder)
		{
			FString ToolIdentifier;
			ToolClass->GetClassPathName().ToString(ToolIdentifier);
			GetToolManager(EToolsContextScope::EdMode)->UnregisterToolType(ToolIdentifier);

			if (Toolkit.IsValid() && FScriptableToolsEditorModeManagerCommands::IsRegistered())
			{
				bool bFoundToolCommand = false;
				const FScriptableToolsEditorModeManagerCommands& ToolManagerCommands = FScriptableToolsEditorModeManagerCommands::Get();
				TSharedPtr<FUICommandInfo> ToolCommand = ToolManagerCommands.FindToolByName(ToolIdentifier, bFoundToolCommand);

				if (ToolCommand)
				{
					const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
					CommandList->UnmapAction(ToolCommand);
				}
			}
		});

		FScriptableToolsEditorModeManagerCommands::Unregister();
		
		if (Toolkit.IsValid())
		{
			FScriptableToolsEditorModeToolkit* ModeToolkit = (FScriptableToolsEditorModeToolkit*)Toolkit.Get();
			ModeToolkit->StartAsyncToolLoading();
		};
	};

	auto RegisterTools = [this]()
	{
		FScriptableToolsEditorModeManagerCommands::Register();

		// Rebind mode commands.
		BindCommands();

		auto GetToolIconName = [](const FString& ToolIdentifier)
		{
			return FName(ToolIdentifier + ".Icon");
		};

		// Register tool icons
		FScriptableToolsEditorModeStyle::FIconTextureMap IconTextureMap;
		ScriptableTools->ForEachScriptableTool([this, GetToolIconName, &IconTextureMap](UClass* ToolClass, UInteractiveToolBuilder* ToolBuilder)
		{
			FString ToolIdentifier;
			ToolClass->GetClassPathName().ToString(ToolIdentifier);

			if (UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(ToolClass->GetDefaultObject()))
			{
				if (ToolCDO->ToolIconTexture)
				{
					IconTextureMap.Emplace(GetToolIconName(ToolIdentifier), ToolCDO->ToolIconTexture);
				}
			}
		});
		if (IconTextureMap.Num() > 0)
		{
			FScriptableToolsEditorModeStyle::RegisterIconTextures(IconTextureMap);
		}

		// Register tools with ToolManager
		FScriptableToolsEditorModeManagerCommands& ToolManagerCommands = const_cast<FScriptableToolsEditorModeManagerCommands&>(FScriptableToolsEditorModeManagerCommands::Get());
		ScriptableTools->ForEachScriptableTool([this, &ToolManagerCommands, GetToolIconName](UClass* ToolClass, UInteractiveToolBuilder* ToolBuilder)
		{
			FString ToolIdentifier;
			ToolClass->GetClassPathName().ToString(ToolIdentifier);
			GetToolManager(EToolsContextScope::EdMode)->RegisterToolType(ToolIdentifier, ToolBuilder);

			// Register commands for each tool
			UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(ToolClass->GetDefaultObject());
			if (ToolCDO)
			{
				FString ToolNameString = ToolCDO->ToolName.IsEmpty() ? ToolIdentifier : ToolCDO->ToolName.ToString();
				const FText ToolLabel = ToolCDO->ToolLongName.IsEmpty() ? FText::FromString(ToolNameString) : ToolCDO->ToolLongName;
				const FText ToolTooltip = ToolCDO->ToolTooltip.IsEmpty() ? ToolLabel : ToolCDO->ToolTooltip;
				FSlateIcon ToolIcon;
				if (ToolCDO->ToolIconTexture)
				{
					FName ToolIconName = GetToolIconName(ToolIdentifier);
					ToolIcon = FSlateIcon(FScriptableToolsEditorModeStyle::Get()->GetStyleSetName(), ToolIconName);
				}
				TSharedPtr<FUICommandInfo> ToolCommand = ToolManagerCommands.RegisterCommand(
					FName(ToolIdentifier),
					ToolLabel,
					ToolTooltip,
					ToolIcon,
					EUserInterfaceActionType::ToggleButton,
					FInputChord());

				if (Toolkit.IsValid())
				{
					const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();
					CommandList->MapAction(ToolCommand,
					FExecuteAction::CreateLambda([this, ToolClass, ToolIdentifier]()
					{
						if (GetToolManager()->CanActivateTool(EToolSide::Mouse, ToolIdentifier))
						{
							if (GetToolManager()->SelectActiveToolType(EToolSide::Mouse, ToolIdentifier)) 
							{
								GetToolManager()->ActivateTool(EToolSide::Mouse);
							}
							else
							{
								UE_LOGF(LogTemp, Warning, "FAILED TO SET ACTIVE TOOL TYPE!");
							}
						}
					}),
					FCanExecuteAction::CreateLambda([this, ToolClass, ToolIdentifier]()
					{
						return GetToolManager()->CanActivateTool(EToolSide::Mouse, ToolIdentifier);
					}),
					FIsActionChecked::CreateLambda([this, ToolClass, ToolIdentifier]()
					{
						return GetToolManager()->GetActiveToolName(EToolSide::Mouse) == ToolIdentifier;
					}));
				}
			}
		});

		if (Toolkit.IsValid())
		{
			FScriptableToolsEditorModeToolkit* ModeToolkit = (FScriptableToolsEditorModeToolkit*)Toolkit.Get();
			ModeToolkit->EndAsyncToolLoading();

			// Register LoadPalette commands
			TArray<FName> PaletteNames;
			ModeToolkit->GetActiveToolPaletteNames(PaletteNames);
			for (const FName PaletteName : PaletteNames)
			{
				const FString PaletteNameString = PaletteName.ToString();
				ensure(!PaletteNameString.IsEmpty());
				const FString LoadPaletteString = "LoadPalette" + PaletteNameString;
				const FText PaletteNameText = FText::FromString(PaletteNameString);
				ToolManagerCommands.RegisterCommand(
					FName(LoadPaletteString),
					PaletteNameText,
					PaletteNameText,
					FSlateIcon(),
					EUserInterfaceActionType::ToggleButton,
					FInputChord());
			}

			ToolManagerCommands.NotifyCommandsChanged();

			ModeToolkit->ForceToolPaletteRebuild();
		}
	};

	auto ToolLoadingUpdate = [this](TSharedPtr<FStreamableHandle> Handle)
	{
		if (Toolkit.IsValid())
		{
			FScriptableToolsEditorModeToolkit* ModeToolkit = (FScriptableToolsEditorModeToolkit*)Toolkit.Get();
			ModeToolkit->SetAsyncProgress(Handle->GetProgress());
		}
	};

	// find all the Tool Blueprints
	if (ScriptableTools)
	{
		UScriptableToolsModeCustomizationSettings* ModeSettings = GetMutableDefault<UScriptableToolsModeCustomizationSettings>();
		if (ModeSettings->RegisterAllTools())
		{
			ScriptableTools->ReinitializeScriptableTools(FToolsLoadedDelegate::CreateLambda(UnregisterTools),
														 FToolsLoadedDelegate::CreateLambda(RegisterTools),
														 FToolsLoadingUpdateDelegate::CreateLambda(ToolLoadingUpdate));
		}
		else
		{
			ScriptableTools->ReinitializeScriptableTools(FToolsLoadedDelegate::CreateLambda(UnregisterTools),
														 FToolsLoadedDelegate::CreateLambda(RegisterTools),
														 FToolsLoadingUpdateDelegate::CreateLambda(ToolLoadingUpdate),
														 &ModeSettings->ToolRegistrationFilters);
		}
	}
}

void UScriptableToolsEditorMode::InitializeModeContexts()
{
	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();

	auto AddContextObject = [this, ContextStore](UScriptableToolContextObject* Object)
	{
		if (ensure(ContextStore->AddContextObject(Object)))
		{
			ContextsToShutdown.Add(Object);
		}
		ContextsToUpdateOnToolEnd.Add(Object);
	};

	UScriptableToolViewportWidgetAPI* ViewportWidgetAPI = NewObject<UScriptableToolViewportWidgetAPI>();
	ViewportWidgetAPI->Initialize(
		[this](TSharedRef<SWidget> InOverlaidWidget) {
			if (Toolkit.IsValid() && Toolkit->IsHosted())
			{
				Toolkit->GetToolkitHost()->AddViewportOverlayWidget(InOverlaidWidget);
			}
		},
		[this](TSharedRef<SWidget> InOverlaidWidget) {
			if (Toolkit.IsValid() && Toolkit->IsHosted())
			{
				Toolkit->GetToolkitHost()->RemoveViewportOverlayWidget(InOverlaidWidget);
			}
		}
		);
#if WITH_UNREAL_DEVELOPER_TOOLS
	ViewportWidgetAPI->MakeDraggableWidgetFunc = [](TSharedRef<SWidget> Content)
		{
			return SNew(UE::ToolWidgets::SDraggableBoxOverlay)
				[
					Content
				];
		};
#endif
	AddContextObject(ViewportWidgetAPI);

	UScriptableToolViewportFocusAPI* ViewportFocusAPI = NewObject<UScriptableToolViewportFocusAPI>();
	ViewportFocusAPI->Initialize(
		[this]() {
			SetFocusInViewport();
		}
		);
	AddContextObject(ViewportFocusAPI);
}

void UScriptableToolsEditorMode::SetFocusInViewport()
{
	FEditorModeTools& ModeManager = GLevelEditorModeTools();

	const FEditorViewportClient* ViewportClient = ModeManager.GetHoveredViewportClient();
	if (!ViewportClient)
	{
		ViewportClient = ModeManager.GetFocusedViewportClient();
	}
	if (!ViewportClient)
	{
		return;
	}

	TSharedPtr<SEditorViewport> EditorViewport = ViewportClient->GetEditorViewportWidget();
	if (!EditorViewport)
	{
		return;
	}

	TSharedPtr<FSceneViewport> SceneViewport = EditorViewport->GetSceneViewport();
	if (!SceneViewport)
	{
		return;
	}

	// set focus to viewport so that hotkeys are immediately detected
	if(const TSharedPtr<SViewport> ViewportWidget = SceneViewport->GetViewportWidget().Pin())
	{
		if (TSharedPtr<SWidget> ViewportContents = ViewportWidget->GetContent())
		{
			FSlateApplication::Get().ForEachUser([&ViewportContents](FSlateUser& User)
			{
				User.SetFocus(ViewportContents.ToSharedRef());
			});
		}
	}
}

void UScriptableToolsEditorMode::OnBlueprintPreCompile(UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}
	
	if (Blueprint->GeneratedClass)
	{
		if (UInteractiveTool* ActiveTool = GetToolManager()->GetActiveTool(EToolSide::Left))
		{
			if (ActiveTool->IsA(Blueprint->GeneratedClass))
			{
				GetToolManager()->DeactivateTool(EToolSide::Left, EToolShutdownType::Cancel);
			}
		}
	}

	// If this BP is a ScriptableInteractiveTool, schedule a rebuild of the toolset on tick.
	// The OnTick is crucial in case the BP compile was initiated during Tool->Setup() for example
	// which expects the tool to continue to exist after the Setup call. Invoking a rebuild
	// inline would force deactivate all tools and result in a crash.
	const UClass* NativeParentClass = FBlueprintEditorUtils::FindFirstNativeClass(Blueprint->ParentClass);
	if (NativeParentClass->IsChildOf(UScriptableInteractiveTool::StaticClass()))
	{
		bRebuildScriptableToolSetOnTick = true;
	}
}

void UScriptableToolsEditorMode::Exit()
{
	UScriptableToolsModeCustomizationSettings* ModeSettings = GetMutableDefault<UScriptableToolsModeCustomizationSettings>();
	ModeSettings->OnSettingChanged().RemoveAll(this);
	
	GEditor->OnBlueprintPreCompile().Remove(BlueprintPreCompileHandle);

	// exit any exclusive active tools w/ cancel
	if (UInteractiveTool* ActiveTool = GetToolManager()->GetActiveTool(EToolSide::Left))
	{
		if (Cast<IInteractiveToolExclusiveToolAPI>(ActiveTool))
		{
			GetToolManager()->DeactivateTool(EToolSide::Left, EToolShutdownType::Cancel);
		}
	}

	UE::Geometry::DeregisterSceneSnappingManager(GetInteractiveToolsContext());
	UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(GetInteractiveToolsContext());


	// deregister transform gizmo context object
	UE::TransformGizmoUtil::DeregisterTransformGizmoContextObject(GetInteractiveToolsContext());
	
	// clear realtime viewport override
	ConfigureRealTimeViewportsOverride(false);

	UContextObjectStore* ContextStore = GetInteractiveToolsContext()->ToolManager->GetContextObjectStore();
	for (TWeakObjectPtr<UScriptableToolContextObject> Context : ContextsToShutdown)
	{
		if (Context.IsValid())
		{
			Context->Shutdown();
			ContextStore->RemoveContextObject(Context.Get());
		}
	}

	// Explicitly unload all tools from the set, just in case
	ScriptableTools->UnloadAllTools();
	ScriptableTools = nullptr;

	// Call base Exit method to ensure proper cleanup
	UEdMode::Exit();
}

void UScriptableToolsEditorMode::OnToolsContextRender(IToolsContextRenderAPI* RenderAPI)
{
}

bool UScriptableToolsEditorMode::ShouldToolStartBeAllowed(const FString& ToolIdentifier) const
{
	if (UInteractiveToolManager* Manager = GetToolManager())
	{
		if (UInteractiveTool* Tool = Manager->GetActiveTool(EToolSide::Left))
		{
			IInteractiveToolExclusiveToolAPI* ExclusiveAPI = Cast<IInteractiveToolExclusiveToolAPI>(Tool);
			if (ExclusiveAPI)
			{
				return false;
			}
		}
	}
	return Super::ShouldToolStartBeAllowed(ToolIdentifier);
}



void UScriptableToolsEditorMode::CreateToolkit()
{
	Toolkit = MakeShareable(new FScriptableToolsEditorModeToolkit);
}

void UScriptableToolsEditorMode::OnToolPostBuild(
	UInteractiveToolManager* InToolManager, EToolSide InSide, 
	UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState)
{
	if (UScriptableInteractiveTool* ScriptTool = Cast<UScriptableInteractiveTool>(InBuiltTool))
	{
		ScriptTool->SetTargetWorld(GetInteractiveToolsContext()->GetWorld());
	}
}

void UScriptableToolsEditorMode::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	// disable slate throttling so that Tool background computes responding to sliders can properly be processed
	// on Tool Tick. Otherwise, when a Tool kicks off a background update in a background thread, the computed
	// result will be ignored until the user moves the slider, ie you cannot hold down the mouse and wait to see
	// the result. This apparently broken behavior is currently by-design.
	FSlateThrottleManager::Get().DisableThrottle(true);
}

void UScriptableToolsEditorMode::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	// re-enable slate throttling (see OnToolStarted)
	FSlateThrottleManager::Get().DisableThrottle(false);

	for (TWeakObjectPtr<UScriptableToolContextObject> Context : ContextsToUpdateOnToolEnd)
	{
		if (Context.IsValid())
		{
			Context->OnToolEnded(Tool);
		}
	}
}

void UScriptableToolsEditorMode::BindCommands()
{
	if (!Toolkit.IsValid())
	{
		return;
	}

	// On subsequent runs of ScriptableToolsEditor mode, the FScriptableToolsEditorModeManagerCommands
	// might be left in an unregistered state. Since this is only initially registered in the
	// ScriptableToolsEditorModeModule, we need to potentially re-register the commands to handle this.
	//
	// With BindCommands being invoked during UEdMode::Enter(), this should ensure that the commands are
	// in a valid state on mode enter.
	if (!FScriptableToolsEditorModeManagerCommands::IsRegistered())
	{
		FScriptableToolsEditorModeManagerCommands::Register();
	}
	
	const FScriptableToolsEditorModeManagerCommands& ToolManagerCommands = FScriptableToolsEditorModeManagerCommands::Get();
	const TSharedRef<FUICommandList>& CommandList = Toolkit->GetToolkitCommands();

	CommandList->MapAction(
		ToolManagerCommands.AcceptActiveTool,
		FExecuteAction::CreateLambda([this]() { 
			GetInteractiveToolsContext()->EndTool(EToolShutdownType::Accept); 
		}),
		FCanExecuteAction::CreateLambda([this]() { return GetInteractiveToolsContext()->CanAcceptActiveTool(); }),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->ActiveToolHasAccept(); }),
		EUIActionRepeatMode::RepeatDisabled
		);

	CommandList->MapAction(
		ToolManagerCommands.CancelActiveTool,
		FExecuteAction::CreateLambda([this]() { GetInteractiveToolsContext()->EndTool(EToolShutdownType::Cancel); }),
		FCanExecuteAction::CreateLambda([this]() { return GetInteractiveToolsContext()->CanCancelActiveTool(); }),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->ActiveToolHasAccept(); }),
		EUIActionRepeatMode::RepeatDisabled
		);

	CommandList->MapAction(
		ToolManagerCommands.CompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { GetInteractiveToolsContext()->EndTool(EToolShutdownType::Completed); }),
		FCanExecuteAction::CreateLambda([this]() { return GetInteractiveToolsContext()->CanCompleteActiveTool(); }),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateLambda([this]() {return GetInteractiveToolsContext()->CanCompleteActiveTool(); }),
		EUIActionRepeatMode::RepeatDisabled
		);

	// These aren't activated by buttons but have default chords that bind the keypresses to the action.
	CommandList->MapAction(
		ToolManagerCommands.AcceptOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { AcceptActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
				return GetInteractiveToolsContext()->CanAcceptActiveTool() || GetInteractiveToolsContext()->CanCompleteActiveTool();
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);

	CommandList->MapAction(
		ToolManagerCommands.CancelOrCompleteActiveTool,
		FExecuteAction::CreateLambda([this]() { CancelActiveToolActionOrTool(); }),
		FCanExecuteAction::CreateLambda([this]() {
				return GetInteractiveToolsContext()->CanCompleteActiveTool() || GetInteractiveToolsContext()->CanCancelActiveTool();
			}),
		FGetActionCheckState(),
		FIsActionButtonVisible(),
		EUIActionRepeatMode::RepeatDisabled);
}


void UScriptableToolsEditorMode::AcceptActiveToolActionOrTool()
{
	// if we have an active Tool that implements 
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedAcceptCommand() && CancelAPI->CanCurrentlyNestedAccept())
		{
			bool bAccepted = CancelAPI->ExecuteNestedAcceptCommand();
			if (bAccepted)
			{
				return;
			}
		}
	}

	const EToolShutdownType ShutdownType = GetInteractiveToolsContext()->CanAcceptActiveTool() ? EToolShutdownType::Accept : EToolShutdownType::Completed;
	GetInteractiveToolsContext()->EndTool(ShutdownType);
}


void UScriptableToolsEditorMode::CancelActiveToolActionOrTool()
{
	// if we have an active Tool that implements 
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolNestedAcceptCancelAPI* CancelAPI = Cast<IInteractiveToolNestedAcceptCancelAPI>(Tool);
		if (CancelAPI && CancelAPI->SupportsNestedCancelCommand() && CancelAPI->CanCurrentlyNestedCancel())
		{
			bool bCancelled = CancelAPI->ExecuteNestedCancelCommand();
			if (bCancelled)
			{
				return;
			}
		}
	}

	const EToolShutdownType ShutdownType = GetInteractiveToolsContext()->CanCancelActiveTool() ? EToolShutdownType::Cancel : EToolShutdownType::Completed;
	GetInteractiveToolsContext()->EndTool(ShutdownType);
}


bool UScriptableToolsEditorMode::ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const
{
	auto ProcessFocusBoxFunc = [](FBox& FocusBoxInOut)
	{
		double MaxDimension = FocusBoxInOut.GetExtent().GetMax();
		double ExpandAmount = (MaxDimension > SMALL_NUMBER) ? (MaxDimension * 0.2) : 25;		// 25 is a bit arbitrary here...
		FocusBoxInOut = FocusBoxInOut.ExpandBy(MaxDimension * 0.2);
	};

	// if Tool supports custom Focus box, use that
	if (GetToolManager()->HasAnyActiveTool())
	{
		UInteractiveTool* Tool = GetToolManager()->GetActiveTool(EToolSide::Mouse);
		IInteractiveToolCameraFocusAPI* FocusAPI = Cast<IInteractiveToolCameraFocusAPI>(Tool);
		if (FocusAPI && FocusAPI->SupportsWorldSpaceFocusBox() )
		{
			InOutBox = FocusAPI->GetWorldSpaceFocusBox();
			if (InOutBox.IsValid)
			{
				ProcessFocusBoxFunc(InOutBox);
				return true;
			}
		}
	}

	// fallback to base focus behavior
	return false;
}


bool UScriptableToolsEditorMode::GetPivotForOrbit(FVector& OutPivot) const
{
	if (GCurrentLevelEditingViewportClient)
	{
		OutPivot = GCurrentLevelEditingViewportClient->GetViewTransform().GetLookAt();
		return true;
	}
	return false;
}



void UScriptableToolsEditorMode::ConfigureRealTimeViewportsOverride(bool bEnable)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_ScriptableToolsMode", "ScriptableTools Mode");
				if (bEnable)
				{
					Viewport.AddRealtimeOverride(bEnable, SystemDisplayName);
				}
				else
				{
					Viewport.RemoveRealtimeOverride(SystemDisplayName, false);
				}
			}
		}
	}
}



#undef LOCTEXT_NAMESPACE
