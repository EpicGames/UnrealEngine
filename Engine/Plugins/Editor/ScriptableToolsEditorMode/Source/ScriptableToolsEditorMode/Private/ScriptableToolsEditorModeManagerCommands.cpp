// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolsEditorModeManagerCommands.h"
#include "ScriptableToolsEditorModeStyle.h"
#include "Styling/ISlateStyle.h"

#define LOCTEXT_NAMESPACE "ScriptableToolsEditorModeManagerCommands"



FScriptableToolsEditorModeManagerCommands::FScriptableToolsEditorModeManagerCommands() :
	TCommands<FScriptableToolsEditorModeManagerCommands>(
		"ScriptableToolsEditorModeToolCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "ScriptableToolsEditorModeToolCommands", "ScriptableTools Mode - Tools"), // Localized context name for displaying
		NAME_None, // Parent
		FScriptableToolsEditorModeStyle::Get()->GetStyleSetName() // Icon Style Set
		)
{
}



TSharedPtr<FUICommandInfo> FScriptableToolsEditorModeManagerCommands::FindToolByName(const FString& Name, bool& bFound) const
{
	bFound = false;
	for (const FStartToolCommand& Command : RegisteredTools)
	{
		if (Command.ToolUIName.Equals(Name, ESearchCase::IgnoreCase)
		 || (Command.ToolCommand.IsValid() && Command.ToolCommand->GetLabel().ToString().Equals(Name, ESearchCase::IgnoreCase)))
		{
			bFound = true;
			return Command.ToolCommand;
		}
	}
	return TSharedPtr<FUICommandInfo>();
}

void FScriptableToolsEditorModeManagerCommands::NotifyCommandsChanged() const
{
	if (IsRegistered())
	{
		CommandsChanged.Broadcast(Get());
	}
}

TSharedPtr<FUICommandInfo> FScriptableToolsEditorModeManagerCommands::RegisterCommand(
		FName InCommandName,
		const FText& InCommandLabel,
		const FText& InCommandTooltip,
		const FSlateIcon& InIcon,
		const EUserInterfaceActionType InUserInterfaceType,
		const FInputChord& InDefaultChord)
{
	const TSharedRef<FBindingContext> SharedThis = AsShared();
	TSharedPtr<FUICommandInfo> Result;
	FUICommandInfo::MakeCommandInfo(
		SharedThis,
		Result,
		InCommandName,
		InCommandLabel,
		InCommandTooltip,
		InIcon,
		InUserInterfaceType,
		InDefaultChord);
	RegisteredTools.Add(FStartToolCommand{ InCommandName.ToString(), Result});
	return Result;
}

void FScriptableToolsEditorModeManagerCommands::RegisterCommands()
{
	// this has to be done with a compile-time macro because UI_COMMAND expands to LOCTEXT macros
#define REGISTER_TOOL_COMMAND(ToolCommandInfo, ToolName, ToolTip) \
		UI_COMMAND(ToolCommandInfo, ToolName, ToolTip, EUserInterfaceActionType::ToggleButton, FInputChord()); \
		RegisteredTools.Add(FStartToolCommand{ ToolName, ToolCommandInfo });

	UI_COMMAND(AcceptActiveTool, "Accept", "Accept the active tool", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CancelActiveTool, "Cancel", "Cancel the active tool", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CompleteActiveTool, "Done", "Complete the active tool", EUserInterfaceActionType::Button, FInputChord());

	// Note that passing a chord into one of these calls hooks the key press to the respective action. 
	UI_COMMAND(AcceptOrCompleteActiveTool, "Accept or Complete", "Accept or complete the active tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter));
	UI_COMMAND(CancelOrCompleteActiveTool, "Cancel or Complete", "Cancel or complete the active tool", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));
	
#undef REGISTER_TOOL_COMMAND
}




#undef LOCTEXT_NAMESPACE
