// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolsEditorModeModule.h"
#include "Modules/ModuleManager.h"
#include "ScriptableToolsEditorModeManagerCommands.h"
#include "ScriptableToolsEditorModeStyle.h"
#include "Misc/CoreDelegates.h"
#include "UI/ScriptableToolGroupSetCustomization.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FScriptableToolsEditorModeModule"


static const FName PropertyEditorModuleName("PropertyEditor");
static const FName ScriptableToolGroupSetName("ScriptableToolGroupSet");


void FScriptableToolsEditorModeModule::StartupModule()
{
	// This code will execute after your module is loaded into memory; the exact timing is specified in the .uplugin file per-module

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FScriptableToolsEditorModeModule::OnPostEngineInit);


	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);
	PropertyModule.RegisterCustomPropertyTypeLayout(ScriptableToolGroupSetName, FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FScriptableToolGroupSetCustomization::MakeInstance));
}

void FScriptableToolsEditorModeModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.

	FScriptableToolsEditorModeManagerCommands::Unregister();
	FScriptableToolsEditorModeStyle::Shutdown();
}

void FScriptableToolsEditorModeModule::OnPostEngineInit()
{
	FScriptableToolsEditorModeStyle::Initialize();
	FScriptableToolsEditorModeManagerCommands::Register();
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FScriptableToolsEditorModeModule, ScriptableToolsEditorMode)
