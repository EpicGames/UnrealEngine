// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolsEditorModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScriptableToolsEditorModeSettings)


#define LOCTEXT_NAMESPACE "ScriptableToolsEditorModeSettings"


FText UScriptableToolsModeCustomizationSettings::GetSectionText() const
{
	return LOCTEXT("ScriptableModeSettingsName", "Scriptable Tools Mode");
}

FText UScriptableToolsModeCustomizationSettings::GetSectionDescription() const
{
	return LOCTEXT("ScriptableModeSettingsDescription", "Configure the Scriptable Tools Editor Mode plugin");
}


#undef LOCTEXT_NAMESPACE