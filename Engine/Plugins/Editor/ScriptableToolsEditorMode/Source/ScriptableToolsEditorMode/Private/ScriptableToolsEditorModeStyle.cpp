// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolsEditorModeStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/Paths.h"
#include "Styling/SlateTypes.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/StyleColors.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FScriptableToolsEditorModeStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(FScriptableToolsEditorModeStyle::InContent( RelativePath, ".svg"), __VA_ARGS__)
#define RootToContentDir StyleSet->RootToContentDir

TMap<FName, FSlateBrush*> FScriptableToolsEditorModeStyle::IconTextureBrushes;

FString FScriptableToolsEditorModeStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("ScriptableToolsEditorMode"))->GetContentDir();
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr< FSlateStyleSet > FScriptableToolsEditorModeStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FScriptableToolsEditorModeStyle::Get() { return StyleSet; }

FName FScriptableToolsEditorModeStyle::GetStyleSetName()
{
	static FName ScriptableToolsEditorModeStyleName(TEXT("ScriptableToolsEditorModeStyle"));
	return ScriptableToolsEditorModeStyleName;
}

const FSlateBrush* FScriptableToolsEditorModeStyle::GetBrush(FName PropertyName, const ANSICHAR* Specifier)
{
	return Get()->GetBrush(PropertyName, Specifier);
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FScriptableToolsEditorModeStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon28x28(28.0f, 28.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);
	const FVector2D Icon120(120.0f, 120.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/ScriptableToolsEditorMode/Content"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	const FTextBlockStyle& NormalText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");

	{
		StyleSet->Set("LevelEditor.ScriptableToolsEditorMode", new IMAGE_PLUGIN_BRUSH("Icons/ScriptableToolsEditorMode_Icon_40x", FVector2D(20.0f, 20.0f)));

		StyleSet->Set("ScriptableToolsEditorModeToolCommands.DefaultToolIcon", new IMAGE_PLUGIN_BRUSH("Icons/Tool_DefaultIcon_40px", Icon20x20));
		StyleSet->Set("ScriptableToolsEditorModeToolCommands.DefaultToolIcon.Small", new IMAGE_PLUGIN_BRUSH("Icons/Tool_DefaultIcon_40px", Icon20x20));

	}

	StyleSet->Set("ToolPalette.MenuIndicator", new IMAGE_PLUGIN_BRUSH_SVG("Icons/chevron-right", Icon20x20, FStyleColors::Foreground));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};


bool FScriptableToolsEditorModeStyle::TryRegisterCustomIcon(FName StyleIdentifier, FString FileNameWithPath, FString ExternalRelativePath)
{
	if ( ! ensure(StyleSet.IsValid()) )
	{
		return false;
	}

	const FVector2D Icon20x20(20.0f, 20.0f);

	if (ExternalRelativePath.IsEmpty())
	{
		ExternalRelativePath = FileNameWithPath;
	}

	if (FPaths::FileExists(FileNameWithPath) == false)
	{
		UE_LOGF(LogTemp, Warning, "Custom ScriptableTool Icon file not found at path %ls (on-disk path is %ls)", *ExternalRelativePath, *FileNameWithPath);
		return false;
	}

	FString Extension = FPaths::GetExtension(FileNameWithPath);
	if ( !(Extension.Equals("svg", ESearchCase::IgnoreCase) || Extension.Equals("png", ESearchCase::IgnoreCase)) )
	{
		UE_LOGF(LogTemp, Warning, "Custom ScriptableTool Icon at path %ls has unsupported type, must be svg or png", *ExternalRelativePath);
		return false;
	}

	// need to re-register style to be able to modify it
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());

	if (Extension.Equals("svg", ESearchCase::IgnoreCase))
	{
		StyleSet->Set(StyleIdentifier,
			new FSlateVectorImageBrush(FileNameWithPath, Icon20x20));
	}
	else if (Extension.Equals("png", ESearchCase::IgnoreCase))
	{
		StyleSet->Set(StyleIdentifier,
			new FSlateImageBrush(FileNameWithPath, Icon20x20) );
	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());

	return true;
}

bool FScriptableToolsEditorModeStyle::RegisterIconTexture(FName StyleIdentifier, TObjectPtr<UTexture2D> InTexture)
{
	if (!InTexture)
	{
		return false;
	}

	RegisterIconTextures({{StyleIdentifier, InTexture}});
	return true;
}

void FScriptableToolsEditorModeStyle::RegisterIconTextures(const FIconTextureMap& InIcons)
{
	// need to re-register style to be able to modify it
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());

	const FVector2D Icon20x20(20.0f, 20.0f);
	for (const TPair<FName, TObjectPtr<UTexture2D>>& IconData : InIcons)
	{
		if (IconData.Value)
		{
			if (FSlateBrush** OldBrush = IconTextureBrushes.Find(IconData.Key))
			{
				delete *OldBrush;
			}
			FSlateBrush* NewBrush = new FSlateImageBrush(static_cast<UObject*>(IconData.Value), Icon20x20);
			IconTextureBrushes.Add(IconData.Key, NewBrush);
			StyleSet->Set(IconData.Key, NewBrush);
		}
	}
	
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}



END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_PLUGIN_BRUSH_SVG
#undef RootToContentDir

void FScriptableToolsEditorModeStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}

	for (TPair<FName, FSlateBrush*> IconBrush : IconTextureBrushes)
	{
		delete IconBrush.Value;
	}
	IconTextureBrushes.Reset();
}
