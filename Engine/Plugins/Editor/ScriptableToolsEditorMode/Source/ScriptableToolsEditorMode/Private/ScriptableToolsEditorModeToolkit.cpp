// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScriptableToolsEditorModeToolkit.h"
#include "Tools/EdModeInteractiveToolsContext.h"
#include "ScriptableToolsEditorMode.h"
#include "ScriptableToolsEditorModeManagerCommands.h"
#include "ScriptableToolsEditorModeStyle.h"

#include "SSingleObjectDetailsPanel.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IDetailsView.h"
#include "Toolkits/AssetEditorModeUILayer.h"
#include "Interfaces/IPluginManager.h"

#include "InteractiveToolManager.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "SPrimaryButton.h"
#include "SAssetDropTarget.h"

#include "ScriptableInteractiveTool.h"
#include "ScriptableToolSet.h"
#include "ScriptableToolsEditorModeSettings.h"

#include "UI/SScriptableToolGroupSetCombo.h"

#include "Engine/Blueprint.h"

#define LOCTEXT_NAMESPACE "FScriptableToolsEditorModeToolkit"

static const FName CustomToolsTabName(TEXT("Custom"));

class SToolPaletteLoadBar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SToolPaletteLoadBar) {}
	SLATE_END_ARGS()

private:

	FScriptableToolsEditorModeToolkit* Toolkit;

	EVisibility IsVisible() const
	{
		if (Toolkit)
		{
			return Toolkit->AreToolsLoading() ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return EVisibility::Collapsed;
	}

	TOptional<float> GetPercentLoaded() const
	{
		if (Toolkit)
		{
			return Toolkit->GetToolPercentLoaded();
		}
		return TOptional<float>();
	}

public:

	void Construct(const FArguments& InArgs, FScriptableToolsEditorModeToolkit* ToolkitIn)
	{
		Toolkit = ToolkitIn;

		ChildSlot
		[
			SNew(SBox)
			.Visibility(this, &SToolPaletteLoadBar::IsVisible)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.f, 6.f)
					//.AutoWidth()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ScriptableToolsLoadingText", "Loading tools..."))
					]
				]
				+SVerticalBox::Slot()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.Padding(2.f, 6.f)
					//.AutoWidth()
					[
						SNew(SProgressBar)
						.BorderPadding(FVector2D::ZeroVector)
						.Percent(this, &SToolPaletteLoadBar::GetPercentLoaded)
						.FillColorAndOpacity(FSlateColor(FLinearColor(0.0f, 1.0f, 1.0f)))
					]
				]
			]

		];

	}

};

class SToolPaletteTagPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SToolPaletteTagPanel) {}
	SLATE_END_ARGS()

private:
	// Pointer back to owning sprite editor instance (the keeper of state)

	UScriptableToolsModeCustomizationSettings* ModeSettings;
	FScriptableToolsEditorModeToolkit* Toolkit;

	TSharedPtr<SScriptableToolGroupSetCombo> TagCombo;
	TSharedPtr<SComboButton> ToolMenuButton;

	FDelegateHandle SettingsUpdateHandle;

	void RefreshData(UObject*, FPropertyChangedEvent& )
	{
		if (TagCombo)
		{
			TagCombo->ForceUpdate();
		}
	}

public:

	virtual ~SToolPaletteTagPanel()
	{
		ModeSettings = GetMutableDefault<UScriptableToolsModeCustomizationSettings>();
		ModeSettings->OnSettingChanged().Remove(SettingsUpdateHandle);
	}

	void Construct(const FArguments& InArgs, FScriptableToolsEditorModeToolkit* ToolkitIn)
	{
		ModeSettings = GetMutableDefault<UScriptableToolsModeCustomizationSettings>();
		Toolkit = ToolkitIn;

		SettingsUpdateHandle = ModeSettings->OnSettingChanged().AddSP(this, &SToolPaletteTagPanel::RefreshData);

		ChildSlot
		[
			SAssignNew(ToolMenuButton, SComboButton)
			.HasDownArrow(false)
			.CollapseMenuOnParentFocus(false)
			.MenuPlacement(EMenuPlacement::MenuPlacement_MenuRight)
			.OnMenuOpenChanged_Lambda([this](bool bOpened)
			{
				FSlateThrottleManager::Get().DisableThrottle(bOpened);
			})
			.ButtonContent()
			[
				SNew(SAssetDropTarget)
				.bSupportsMultiDrop(true)
				.OnAreAssetsAcceptableForDropWithReason_Lambda([this](TArrayView<FAssetData> InAssets, FText& OutReason)
				{
					if (InAssets.Num() > 1)
					{
						OutReason = FText(LOCTEXT("ScriptableToolPaletteTagDropWarningPlural", "Assets must be Scriptable Tool Tags."));
					}
					else
					{
						OutReason = FText(LOCTEXT("ScriptableToolPaletteTagDropWarning", "Asset must be a Scriptable Tool Tag."));
					}

					for (FAssetData& Asset : InAssets)
					{
						UObject* AssetObject = Asset.GetAsset();

						if (AssetObject == nullptr)
						{
							return false;
						}

						if (!AssetObject->IsA(UBlueprint::StaticClass()))
						{
							return false;
						}

						UBlueprint* BlueprintObject = Cast<UBlueprint>(AssetObject);

						if (!BlueprintObject->GeneratedClass->IsChildOf(UScriptableToolGroupTag::StaticClass()))
						{
							return false;
						}
					}
					return true;
				})
				.OnAssetsDropped_Lambda([this](const FDragDropEvent&, TArrayView<FAssetData> InAssets)
					{
						ModeSettings->PreEditChange(UScriptableToolsModeCustomizationSettings::StaticClass()->FindPropertyByName("bRegisterAllTools"));
						ModeSettings->PreEditChange(UScriptableToolsModeCustomizationSettings::StaticClass()->FindPropertyByName("ToolRegistrationFilters"));
						for (FAssetData& Asset : InAssets)
						{
							UObject* AssetObject = Asset.GetAsset();

							UBlueprint* BlueprintObject = Cast<UBlueprint>(AssetObject);

							UClass* BlueprintClass = BlueprintObject->GetBlueprintClass();							

							if (BlueprintObject->GeneratedClass->IsChildOf(UScriptableToolGroupTag::StaticClass()))
							{
								TSubclassOf<UScriptableToolGroupTag> TagSubclass{ BlueprintObject->GeneratedClass };
								FScriptableToolGroupSet::FGroupSet  Groups = ModeSettings->ToolRegistrationFilters.GetGroups();
								Groups.Add(TagSubclass);
								ModeSettings->ToolRegistrationFilters.SetGroups(Groups);
							}
						}
						ModeSettings->PostEditChange();
					})
				[
					SNew(SBorder)
					.Visibility(EVisibility::SelfHitTestInvisible)
					.Padding(FMargin(0.0f))
					.BorderImage(FAppStyle::Get().GetBrush("ProjectBrowser.ProjectTile.DropShadow"))
					[
						SNew(SOverlay)
						+ SOverlay::Slot()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							[
								SNew(SBox)
								.HeightOverride(30)
								[
									SNew(SBorder)
									.Padding(0)
									.BorderImage(FStyleDefaults::GetNoBrush())
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)									
									[
										SNew(STextBlock)
										.Text_Lambda([this]() {
											if(ModeSettings->RegisterAllTools())
											{
												return LOCTEXT("ScriptableToolsAllToolsLabel", "Showing all tools");
											}
											else
											{

												TArray<FText> GroupNames;

												for (UClass* GroupClass : ModeSettings->ToolRegistrationFilters.GetGroups())
												{
													if (GroupClass)
													{
														UScriptableToolGroupTag* GroupTag = Cast<UScriptableToolGroupTag>(GroupClass->GetDefaultObject());

														if (GroupTag)
														{
															GroupNames.Add(FText::FromString(GroupTag->Name));
														}
													}
												}

												if (GroupNames.Num() == 0)
												{
													return LOCTEXT("ScriptableToolsZeroGroupLabel", "Showing tools from no groups");
												}
												else if (GroupNames.Num() == 1)
												{
													return FText::Format(LOCTEXT("ScriptableToolsOneGroupLabel", "Showing tools from {0}"), GroupNames[0]);
												}
												else if (GroupNames.Num() == 2)
												{
													return FText::Format(LOCTEXT("ScriptableToolsTwoGroupLabel", "Showing tools from {0} and {1}"), GroupNames[0], GroupNames[1]);
												}
												else
												{
													return FText::Format(LOCTEXT("ScriptableToolsManyGroupLabel", "Showing tools from {0}, {1} and more..."), GroupNames[0], GroupNames[1]);
												}
											}
										})
										.ToolTipText(LOCTEXT("ScriptableToolsGroupButtonTooltip", "Select tool groups or drag tool group asset here to filter displayed tools."))
									]
								]
							]
							+ SHorizontalBox::Slot()
							.FillWidth(1.0)
							[
								SNew(SBox)
								.HeightOverride(30)
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							.HAlign(EHorizontalAlignment::HAlign_Right)
							[
								SNew(SImage)
								.Image(FScriptableToolsEditorModeStyle::Get()->GetBrush("ToolPalette.MenuIndicator"))
							]
						]
					]
				]
			]			
			.MenuContent()
			[
				SNew(SBox)				
				.WidthOverride(300)
				[
					SNew(SBorder)
					.Padding(15)
					.BorderImage(FStyleDefaults::GetNoBrush())
					[

						SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.FillWidth(1.0)
							[
								SAssignNew(TagCombo, SScriptableToolGroupSetCombo)
								.StructPtr(&ModeSettings->ToolRegistrationFilters)
								.OnChanged_Lambda([this]() {
									ModeSettings->PostEditChange();
								})
							]
						]
					]
				]
			]			
		];
	}

	

	// End of SSingleObjectDetailsPanel interface
};


FScriptableToolsEditorModeToolkit::FScriptableToolsEditorModeToolkit()
{
}


FScriptableToolsEditorModeToolkit::~FScriptableToolsEditorModeToolkit()
{
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.RemoveAll(this);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.RemoveAll(this);

	UScriptableToolsModeCustomizationSettings* ModeSettings = GetMutableDefault<UScriptableToolsModeCustomizationSettings>();
	ModeSettings->OnSettingChanged().Remove(SettingsUpdateHandle);	
}


void FScriptableToolsEditorModeToolkit::CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut)
{
	//ArgsInOut.ColumnWidth = 0.3f;
}

void FScriptableToolsEditorModeToolkit::RegisterPalettes()
{
	UEdMode* ScriptableMode = GetScriptableEditorMode().Get();

	ToolkitSections = MakeShared<FToolkitSections>();
	FToolkitBuilderArgs ToolkitBuilderArgs(ScriptableMode->GetModeInfo().ToolbarCustomizationName);
	ToolkitBuilderArgs.ToolkitCommandList = GetToolkitCommands();
	ToolkitBuilderArgs.ToolkitSections = ToolkitSections;
	ToolkitBuilderArgs.SelectedCategoryTitleVisibility = EVisibility::Collapsed;
	// This lets us re-show the buttons if the user clicks a category with a tool still active.
	ToolkitBuilderArgs.CategoryReclickBehavior = FToolkitBuilder::ECategoryReclickBehavior::TreatAsChanged;
	ToolkitBuilder = MakeShared<FToolkitBuilder>(ToolkitBuilderArgs);
	ToolkitBuilder->SetCategoryButtonLabelVisibility(true);

	// Actual population of palettes is deferred to support async load of
	// scriptable tools. Proper population should happen as part of
	// ForceToolPaletteRebuild.
	
	// if selected palette changes, make sure we are showing the palette command buttons, which may be hidden by active Tool
	ActivePaletteChangedHandle = ToolkitBuilder->OnActivePaletteChanged.AddLambda([this]()
	{
		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);
	});
}

void FScriptableToolsEditorModeToolkit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode)
{
	const UScriptableToolsModeCustomizationSettings* UISettings = GetDefault<UScriptableToolsModeCustomizationSettings>();
	bUsesToolkitBuilder = !UISettings->bUseLegacyPalette;

	// Have to create the ToolkitWidget here because FModeToolkit::Init() is going to ask for it and add
	// it to the Mode panel, and not ask again afterwards. However we have to call Init() to get the 
	// ModeDetailsView created, that we need to add to the ToolkitWidget. So, we will create the Widget
	// here but only add the rows to it after we call Init()
	
	TSharedPtr<SVerticalBox> ToolkitWidgetVBox = SNew(SVerticalBox);
	
	if ( !bUsesToolkitBuilder )
	{
		SAssignNew(ToolkitWidget, SBorder)
			.HAlign(HAlign_Fill)
			.Padding(4)
			[
				ToolkitWidgetVBox->AsShared()
			];
	}

	FModeToolkit::Init(InitToolkitHost, InOwningMode);

	GetToolkitHost()->OnActiveViewportChanged().AddSP(this, &FScriptableToolsEditorModeToolkit::OnActiveViewportChanged);

	ModeWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ModeWarningArea->SetText(FText::GetEmpty());
	ModeWarningArea->SetVisibility(EVisibility::Collapsed);

	ModeHeaderArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 12));
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ModeHeaderArea->SetJustification(ETextJustify::Center);


	ToolWarningArea = SNew(STextBlock)
		.AutoWrapText(true)
		.Font(FCoreStyle::GetDefaultFontStyle("Bold", 9))
		.ColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.15f, 0.15f)));
	ToolWarningArea->SetText(FText::GetEmpty());

	ToolPaletteTagPanel = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(1.f)
		.Padding(1.f)
		[
			SNew(SToolPaletteTagPanel, this)
		];
	ToolPaletteTagPanel->SetVisibility(bShowToolTagFilterBar ? EVisibility::Visible : EVisibility::Collapsed);


	ToolPaletteLoadBar = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(1.f)
		.Padding(1.f)
		[
			SNew(SToolPaletteLoadBar, this)
		];

	ToolPaletteHeader = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ToolPaletteTagPanel->AsShared()
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			ToolPaletteLoadBar->AsShared()
		];

	ToolDetailsHeader = SNew(SVerticalBox);

	if (bUsesToolkitBuilder)
	{
		RegisterPalettes();
	}
	else
	{
		// add the various sections to the mode toolbox
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ModeWarningArea->AsShared()
			];
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ModeHeaderArea->AsShared()
			];
		ToolkitWidgetVBox->AddSlot().AutoHeight().HAlign(HAlign_Fill).Padding(5)
			[
				ToolWarningArea->AsShared()
			];
		ToolkitWidgetVBox->AddSlot().HAlign(HAlign_Fill).FillHeight(1.f)
			[
				ModeDetailsView->AsShared()
			];
	}

	ClearNotification();
	ClearWarning();

	if (HasToolkitBuilder())
	{
		ToolkitSections->ModeWarningArea = ModeWarningArea;
		ToolkitSections->DetailsView = ModeDetailsView;
		ToolkitSections->ToolWarningArea = ToolWarningArea;
		ToolkitSections->Header = ToolPaletteHeader;
		ToolkitSections->ToolHeader = ToolDetailsHeader;

		SAssignNew(ToolkitWidget, SBorder)
		.HAlign(HAlign_Fill)
		.Padding(0)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			ToolkitBuilder->GenerateWidget()->AsShared()
		];	
	}

	ActiveToolName = FText::GetEmpty();
	ActiveToolMessage = FText::GetEmpty();

	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolNotificationMessage.AddSP(this, &FScriptableToolsEditorModeToolkit::PostNotification);
	GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->OnToolWarningMessage.AddSP(this, &FScriptableToolsEditorModeToolkit::PostWarning);

	SAssignNew(ViewportOverlayWidget, SHorizontalBox)

	+SHorizontalBox::Slot()
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Bottom)
	.Padding(FMargin(0.0f, 0.0f, 0.f, 15.f))
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewport.OverlayBrush"))
		.Padding(8.f)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(SImage)
				.Image_Lambda([this] () { return ActiveToolIcon; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
			[
				SNew(STextBlock)
				.Text(this, &FScriptableToolsEditorModeToolkit::GetActiveToolDisplayName)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(0.0, 0.f, 2.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("OverlayAccept", "Accept"))
				.ToolTipText(LOCTEXT("OverlayAcceptTooltip", "Accept/Commit the results of the active Tool [Enter]"))
				.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->EndTool(EToolShutdownType::Accept); return FReply::Handled(); })
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanAcceptActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SButton)
				.Text(LOCTEXT("OverlayCancel", "Cancel"))
				.ToolTipText(LOCTEXT("OverlayCancelTooltip", "Cancel the active Tool [Esc]"))
				.HAlign(HAlign_Center)
				.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->EndTool(EToolShutdownType::Cancel); return FReply::Handled(); })
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCancelActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->ActiveToolHasAccept() ? EVisibility::Visible : EVisibility::Collapsed; })
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(2.0, 0.f, 0.f, 0.f))
			[
				SNew(SPrimaryButton)
				.Text(LOCTEXT("OverlayComplete", "Complete"))
				.ToolTipText(LOCTEXT("OverlayCompleteTooltip", "Exit the active Tool [Enter]"))
				.OnClicked_Lambda([this]() { GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->EndTool(EToolShutdownType::Completed); return FReply::Handled(); })
				.IsEnabled_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->CanCompleteActiveTool(); })
				.Visibility_Lambda([this]() { return GetScriptableEditorMode()->GetInteractiveToolsContext(EToolsContextScope::EdMode)->ActiveToolHasAccept() ? EVisibility::Collapsed : EVisibility::Visible; })
			]
		]	
	];

}





void FScriptableToolsEditorModeToolkit::InitializeAfterModeSetup()
{
	if (bFirstInitializeAfterModeSetup)
	{
		bFirstInitializeAfterModeSetup = false;
	}

	UpdateActiveToolCategories();
}



void FScriptableToolsEditorModeToolkit::UpdateActiveToolCategories()
{
	// build tool category list
	ActiveToolCategories.Reset();
	UScriptableToolsEditorMode* EditorMode = Cast<UScriptableToolsEditorMode>(GetScriptableEditorMode());
	UScriptableToolSet* ScriptableTools = EditorMode->GetActiveScriptableTools();
	ScriptableTools->ForEachScriptableTool( [&](UClass* ToolClass, UBaseScriptableToolBuilder* ToolBuilder) 
	{
		UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(ToolClass->GetDefaultObject());
		const bool bIsCategoryEmpty = ToolCDO->ToolCategory.IsEmpty();

		// For legacy UI, don't process empty categories here.
		if (!HasToolkitBuilder() && bIsCategoryEmpty)
		{
			return;
		}

		FName CategoryName = bIsCategoryEmpty ? CustomToolsTabName : FName(ToolCDO->ToolCategory.ToString()); 
		if (ActiveToolCategories.Contains(CategoryName) == false)
		{
			ActiveToolCategories.Add(CategoryName, {});
		}

		if (TArray<FScriptableToolData>* CategoryTools = ActiveToolCategories.Find(CategoryName))
		{
			CategoryTools->Add(FScriptableToolData{ToolCDO->ToolCategory, ToolClass, ToolBuilder});
		}
	});
}



void FScriptableToolsEditorModeToolkit::UpdateActiveToolProperties()
{
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	if (CurTool != nullptr)
	{
		ModeDetailsView->SetObjects(CurTool->GetToolProperties(true));
	}
}

void FScriptableToolsEditorModeToolkit::InvalidateCachedDetailPanelState(UObject* ChangedObject)
{
	ModeDetailsView->InvalidateCachedState();
}


void FScriptableToolsEditorModeToolkit::PostNotification(const FText& Message)
{
	ClearNotification();

	ActiveToolMessage = Message;

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		ActiveToolMessageHandle = GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PushStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessage);
	}
}

void FScriptableToolsEditorModeToolkit::ClearNotification()
{
	ActiveToolMessage = FText::GetEmpty();

	if (ModeUILayer.IsValid())
	{
		TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin();
		GEditor->GetEditorSubsystem<UStatusBarSubsystem>()->PopStatusBarMessage(ModeUILayerPtr->GetStatusBarName(), ActiveToolMessageHandle);
	}
	ActiveToolMessageHandle.Reset();
}


void FScriptableToolsEditorModeToolkit::PostWarning(const FText& Message)
{
	ToolWarningArea->SetText(Message);
	ToolWarningArea->SetVisibility(EVisibility::Visible);
}

void FScriptableToolsEditorModeToolkit::ClearWarning()
{
	ToolWarningArea->SetText(FText());
	ToolWarningArea->SetVisibility(EVisibility::Collapsed);
}



FName FScriptableToolsEditorModeToolkit::GetToolkitFName() const
{
	return FName("ScriptableToolsEditorMode");
}

FText FScriptableToolsEditorModeToolkit::GetBaseToolkitName() const
{
	return NSLOCTEXT("ScriptableToolsEditorModeToolkit", "DisplayName", "ScriptableToolsEditorMode Tool");
}

void FScriptableToolsEditorModeToolkit::GetToolPaletteNames(TArray<FName>& PaletteNames) const
{
	PaletteNames.Reset();
	bool bFoundUncategorized = false;

	UScriptableToolsEditorMode* EditorMode = Cast<UScriptableToolsEditorMode>(GetScriptableEditorMode());
	UScriptableToolSet* ScriptableTools = EditorMode->GetActiveScriptableTools();
	ScriptableTools->ForEachScriptableTool( [&](UClass* ToolClass, UBaseScriptableToolBuilder* ToolBuilder) 
	{
		UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(ToolClass->GetDefaultObject());
		if (ToolCDO->bShowToolInEditor == false)
		{
			return;
		}

		if (ToolCDO->ToolCategory.IsEmpty() == false)
		{
			FName CategoryName(ToolCDO->ToolCategory.ToString());
			if (ActiveToolCategories.Contains(CategoryName))
			{
				PaletteNames.AddUnique(CategoryName);
			}
			else
			{
				bFoundUncategorized = true;
			}
		}
		else
		{
			bFoundUncategorized = true;
		}
	});

	if ( bFoundUncategorized )
	{
		PaletteNames.Add(CustomToolsTabName);
	}
}


FText FScriptableToolsEditorModeToolkit::GetToolPaletteDisplayName(FName Palette) const
{
	return FText::FromName(Palette);
}

void FScriptableToolsEditorModeToolkit::BuildToolPalette(FName PaletteIndex, class FToolBarBuilder& ToolbarBuilder) 
{
	if (HasToolkitBuilder())
	{
		return;
	}
	
	const FScriptableToolsEditorModeManagerCommands& Commands = FScriptableToolsEditorModeManagerCommands::Get();

	static TArray<TSharedPtr<FUIAction>> ActionsHack;

	bool bIsUncategorizedPalette = (PaletteIndex == CustomToolsTabName);

	ActionsHack.Reset();
	
	// this is kind of dumb and we probably should maintain a map <FName, TArray<ToolClass>> when we build the ActiveToolCategories...
	UScriptableToolsEditorMode* EditorMode = Cast<UScriptableToolsEditorMode>(GetScriptableEditorMode());
	UInteractiveToolManager* ToolManager = EditorMode->GetToolManager();
	UScriptableToolSet* ScriptableTools = EditorMode->GetActiveScriptableTools();
	ScriptableTools->ForEachScriptableTool([&](UClass* ToolClass, UBaseScriptableToolBuilder* ToolBuilder)
	{
		UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(ToolClass->GetDefaultObject());
		if (ToolCDO->bShowToolInEditor == false)
		{
			return;
		}

		FName UseCategoryName = ToolCDO->ToolCategory.IsEmpty() ? CustomToolsTabName : FName(ToolCDO->ToolCategory.ToString());
		if (ActiveToolCategories.Contains(UseCategoryName) == false)
		{
			UseCategoryName = CustomToolsTabName;
		}
		if (UseCategoryName != PaletteIndex)
		{
			return;
		}

		
		FString ToolIdentifier;
		ToolClass->GetClassPathName().ToString(ToolIdentifier);

		TSharedPtr<FUIAction> NewAction = MakeShared<FUIAction>(
		FExecuteAction::CreateLambda([this, ToolClass, ToolIdentifier, ToolManager]()
		{
			UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(ToolClass->GetDefaultObject());
			if (ToolManager->SelectActiveToolType(EToolSide::Mouse, ToolIdentifier))
			{
				if (ToolManager->CanActivateTool(EToolSide::Mouse, ToolIdentifier)) 
				{
					bool bLaunched = ToolManager->ActivateTool(EToolSide::Mouse);
					//ensure(bLaunched);
				}
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "FAILED TO SET ACTIVE TOOL TYPE!");
			}

		}),
		FCanExecuteAction::CreateLambda([this, ToolClass, ToolIdentifier, ToolManager]()
		{
			UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(ToolClass->GetDefaultObject());
			if (ToolManager->SelectActiveToolType(EToolSide::Mouse, ToolIdentifier))
			{
				return ToolManager->CanActivateTool(EToolSide::Mouse, ToolIdentifier);
			}
			else
			{
				UE_LOGF(LogTemp, Warning, "FAILED TO SET ACTIVE TOOL TYPE!");
				return false;
			}
		}));

		ActionsHack.Add(NewAction);

		FName InExtensionHook = NAME_None;
		TAttribute<FText> Label = ToolCDO->ToolName.IsEmpty() ? LOCTEXT("EmptyToolName", "Tool") : ToolCDO->ToolName;
		TAttribute<FText> Tooltip = ToolCDO->ToolTooltip.IsEmpty() ? FText() : ToolCDO->ToolTooltip;
		
		// default icon comes with the mode
		TAttribute<FSlateIcon> Icon = FSlateIcon(FScriptableToolsEditorModeStyle::Get()->GetStyleSetName(), "ScriptableToolsEditorModeToolCommands.DefaultToolIcon");

		// if a custom icon is defined, try to find it, this can fail in many ways, in that case
		// the default icon is kept
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FString CustomIconPath = ToolCDO->CustomIconPath;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (CustomIconPath.IsEmpty() == false)
		{
			FName ToolIconToken( FString("ScriptableToolsEditorModeToolCommands.") + ToolIdentifier );

			// Custom Tool Icons are assumed to reside in the same Content folder as the Plugin/Project that
			// the Tool Class is defined in, and that the CustomIconPath is a relative path inside that Content folder.
			// use the class Package path to determine if this it is in a Plugin or directly in the Project, so that
			// we can get the right ContentDir below.
			// (Note that a relative ../../../ style path can always be used to redirect to any other file...)
			FString FullPathName = ToolCDO->GetClass()->GetPathName();
			FString PathPart, FilenamePart, ExtensionPart;
			FPaths::Split(FullPathName, PathPart, FilenamePart, ExtensionPart);

			FString FullIconPath = CustomIconPath;
			if (PathPart.StartsWith("/Game"))
			{
				FullIconPath = FPaths::ProjectContentDir() / CustomIconPath;
			}
			else
			{
				TArray<FString> PathParts;
				PathPart.ParseIntoArray(PathParts, TEXT("/"));
				if (PathParts.Num() > 0)
				{
					FString PluginContentDir = IPluginManager::Get().FindPlugin(PathParts[0])->GetContentDir();
					FullIconPath = PluginContentDir / CustomIconPath;
				}
				else  // something is wrong, fall back to project content dir
				{
					FullIconPath = FPaths::ProjectContentDir() / CustomIconPath;
				}
			}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (FScriptableToolsEditorModeStyle::TryRegisterCustomIcon(ToolIconToken, FullIconPath, CustomIconPath))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				Icon = FSlateIcon(FScriptableToolsEditorModeStyle::Get()->GetStyleSetName(), ToolIconToken);
			}
		}
		
		ToolbarBuilder.AddToolBarButton(*NewAction, InExtensionHook, Label, Tooltip, Icon);

	});
}

void FScriptableToolsEditorModeToolkit::InvokeUI()
{
	FModeToolkit::InvokeUI();

	// FModeToolkit::UpdatePrimaryModePanel() wrapped our GetInlineContent() output in a SScrollBar widget,
	// however this doesn't make sense as we want to dock panels to the "top" and "bottom" of our mode panel area,
	// and the details panel in the middle has it's own scrollbar already. The SScrollBar is hardcoded as the content
	// of FModeToolkit::InlineContentHolder so we can just replace it here
	InlineContentHolder->SetContent(GetInlineContent().ToSharedRef());
}



void FScriptableToolsEditorModeToolkit::ForceToolPaletteRebuild()
{
	this->UpdateActiveToolCategories();

	if (HasToolkitBuilder())
	{
		RebuildModeToolkitBuilderPalettes();
	}
	if (ModeUILayer.IsValid() && HasIntegratedToolPalettes() == false)
	{
		if (TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin())
		{
			TSharedPtr<FUICommandList> CommandList;
			if (GetScriptableEditorMode().IsValid())
			{
				UEdMode* ScriptableMode = GetScriptableEditorMode().Get();
				CommandList = GetToolkitCommands();
				ActiveToolBarRows.Reset();

				TArray<FName> PaletteNames;
				GetToolPaletteNames(PaletteNames);
				for (const FName& Palette : PaletteNames)
				{
					TSharedRef<SWidget> PaletteWidget = CreatePaletteWidget(CommandList, ScriptableMode->GetModeInfo().ToolbarCustomizationName, Palette);
					ActiveToolBarRows.Emplace(ScriptableMode->GetID(), Palette, GetToolPaletteDisplayName(Palette), PaletteWidget);
				}

				RebuildModeToolPaletteWidgets();
			}
		}
	}
}

void FScriptableToolsEditorModeToolkit::GetActiveToolPaletteNames(TArray<FName>& OutPaletteNames)
{
	UpdateActiveToolCategories();
	GetToolPaletteNames(OutPaletteNames);
}

void FScriptableToolsEditorModeToolkit::OnToolPaletteChanged(FName PaletteName) 
{
}

void FScriptableToolsEditorModeToolkit::EnableShowRealtimeWarning(bool bEnable)
{
	if (bShowRealtimeWarning != bEnable)
	{
		bShowRealtimeWarning = bEnable;
		UpdateShowWarnings();
	}
}

void FScriptableToolsEditorModeToolkit::OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	UpdateActiveToolProperties();

	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	CurTool->OnPropertySetsModified.AddSP(this, &FScriptableToolsEditorModeToolkit::UpdateActiveToolProperties);
	CurTool->OnPropertyModifiedDirectlyByTool.AddSP(this, &FScriptableToolsEditorModeToolkit::InvalidateCachedDetailPanelState);

	ModeHeaderArea->SetVisibility(EVisibility::Collapsed);

	ActiveToolName = CurTool->GetToolInfo().ToolDisplayName;
	if (UScriptableInteractiveTool* ScriptableTool = Cast<UScriptableInteractiveTool>(CurTool))
	{
		if ( ScriptableTool->ToolLongName.IsEmpty() == false )
		{
			ActiveToolName = ScriptableTool->ToolLongName;
		}
		else if ( ScriptableTool->ToolName.IsEmpty() == false )
		{
			ActiveToolName = ScriptableTool->ToolName;
		}

		if (HasToolkitBuilder())
		{
			TSharedRef<SWidget> ActiveToolPanelHeaderWidget = ScriptableTool->GetToolPanelToolHeaderWidget(); 
			ToolDetailsHeader->ClearChildren();
			ToolDetailsHeader->AddSlot()
			.AutoHeight()
			[
				ActiveToolPanelHeaderWidget
			];
		}
	}

	if (HasToolkitBuilder())
	{
		ToolkitBuilder->SetActiveToolDisplayName(ActiveToolName);
		if (const UScriptableToolsModeCustomizationSettings* Settings = GetDefault<UScriptableToolsModeCustomizationSettings>())
		{
			if (Settings->bAlwaysShowToolButtons == false)
			{
				ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Collapsed);
			}
		}
	}

	// try to update icon
	FString ActiveToolIdentifier = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveToolName(EToolSide::Left);
	ActiveToolIdentifier.InsertAt(0, ".");
	FName ActiveToolIconName = ISlateStyle::Join(FScriptableToolsEditorModeManagerCommands::Get().GetContextName(), TCHAR_TO_ANSI(*ActiveToolIdentifier));
	ActiveToolIcon = FScriptableToolsEditorModeStyle::Get()->GetOptionalBrush(ActiveToolIconName);

	GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
}

void FScriptableToolsEditorModeToolkit::OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool)
{
	if (IsHosted())
	{
		GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef());
	}

	ModeDetailsView->SetObject(nullptr);
	ActiveToolName = FText::GetEmpty();
	if (HasToolkitBuilder())
	{
		ToolkitBuilder->SetActiveToolDisplayName(FText::GetEmpty());
		ToolkitBuilder->SetActivePaletteCommandsVisibility(EVisibility::Visible);

		ToolDetailsHeader->ClearChildren();
	}
	
	ModeHeaderArea->SetVisibility(EVisibility::Visible);
	ModeHeaderArea->SetText(LOCTEXT("SelectToolLabel", "Select a Tool from the Toolbar"));
	ClearNotification();
	ClearWarning();
	UInteractiveTool* CurTool = GetScriptableEditorMode()->GetToolManager(EToolsContextScope::EdMode)->GetActiveTool(EToolSide::Left);
	if ( CurTool )
	{
		CurTool->OnPropertySetsModified.RemoveAll(this);
		CurTool->OnPropertyModifiedDirectlyByTool.RemoveAll(this);
	}
}

void FScriptableToolsEditorModeToolkit::OnActiveViewportChanged(TSharedPtr<IAssetViewport> OldViewport, TSharedPtr<IAssetViewport> NewViewport)
{
	// Only worry about handling this notification if we have an active tool
	if (!ActiveToolName.IsEmpty())
	{
		// Check first to see if this changed because the old viewport was deleted and if not, remove our hud
		if (OldViewport)	
		{
			GetToolkitHost()->RemoveViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef(), OldViewport);
		}

		// Add the hud to the new viewport
		GetToolkitHost()->AddViewportOverlayWidget(ViewportOverlayWidget.ToSharedRef(), NewViewport);
	}
}

void FScriptableToolsEditorModeToolkit::UpdateShowWarnings()
{
	if (bShowRealtimeWarning )
	{
		if (ModeWarningArea->GetVisibility() == EVisibility::Collapsed)
		{
			ModeWarningArea->SetText(LOCTEXT("ScriptableToolsModeToolkitRealtimeWarning", "Realtime Mode is required for Scriptable Tools to work correctly. Please enable Realtime Mode in the Viewport Options or with the Ctrl+r hotkey."));
			ModeWarningArea->SetVisibility(EVisibility::Visible);
		}
	}
	else
	{
		ModeWarningArea->SetText(FText());
		ModeWarningArea->SetVisibility(EVisibility::Collapsed);
	}

}

void FScriptableToolsEditorModeToolkit::RebuildModeToolBar()
{
	TSharedPtr<SDockTab> ToolbarTabPtr = ModeToolbarTab.Pin();
	if (ToolbarTabPtr && HasToolkitBuilder())
	{
		ToolbarTabPtr->SetParentDockTabStackTabWellHidden(true);
	}

	// If the tab or box is not valid the toolbar has not been opened or has been closed by the user
	TSharedPtr<SVerticalBox> ModeToolbarBoxPinned = ModeToolbarBox.Pin();
	if (ModeToolbarTab.IsValid() && ModeToolbarBoxPinned)
	{
		ModeToolbarBoxPinned->ClearChildren();
		bool bExclusivePalettes = true;
		ToolBoxVBox = SNew(SVerticalBox);

		RebuildModeToolPaletteWidgets();

		ModeToolbarBoxPinned->AddSlot()
		.AutoHeight()
		.Padding(1.f)
		[
			SAssignNew(ToolPaletteTagPanel, SBox)
			[
				SNew(SToolPaletteTagPanel, this)
			]
		];
		ToolPaletteTagPanel->SetVisibility(bShowToolTagFilterBar ? EVisibility::Visible : EVisibility::Collapsed);


		ModeToolbarBoxPinned->AddSlot()
		.AutoHeight()
		.Padding(1.f)
		[
			SNew(SBox)
			[
				SNew(SToolPaletteLoadBar, this)
			]
		];

		ModeToolbarBoxPinned->AddSlot()
		[
			SNew(SScrollBox)
			.Visibility_Lambda([this]()
			{
				return AreToolsLoading() ? EVisibility::Collapsed : EVisibility::Visible;
			})
			+ SScrollBox::Slot()
			[
				ToolBoxVBox.ToSharedRef()
			]
		];
		


	}
}

void FScriptableToolsEditorModeToolkit::StartAsyncToolLoading()
{
	bAsyncLoadInProgress = true;
	AsyncLoadProgress = 0.0;
}

void FScriptableToolsEditorModeToolkit::SetAsyncProgress(float PercentLoaded)
{
	ensure(bAsyncLoadInProgress == true);
	AsyncLoadProgress = PercentLoaded;
}

void FScriptableToolsEditorModeToolkit::EndAsyncToolLoading()
{
	ensure(bAsyncLoadInProgress == true);
	bAsyncLoadInProgress = false;
	AsyncLoadProgress = 1.0;
}

bool FScriptableToolsEditorModeToolkit::AreToolsLoading() const
{
	return bAsyncLoadInProgress;
}

TOptional<float> FScriptableToolsEditorModeToolkit::GetToolPercentLoaded() const
{
	if (bAsyncLoadInProgress)
	{
		return TOptional<float>(AsyncLoadProgress);
	}
	return TOptional<float>();
}

bool FScriptableToolsEditorModeToolkit::ShouldShowModeToolbar() const
{
	// Always return true here, to ensure we render the toolbar even if no tools are loaded via group tags.
	return true;
}

void FScriptableToolsEditorModeToolkit::SetShowToolTagFilterBar(bool bShowToolTagFilterBarIn)
{
	bShowToolTagFilterBar = bShowToolTagFilterBarIn;
	if (ToolPaletteTagPanel)
	{
		ToolPaletteTagPanel->SetVisibility(bShowToolTagFilterBar ? EVisibility::Visible : EVisibility::Collapsed);
	}
}

void FScriptableToolsEditorModeToolkit::RebuildModeToolPaletteWidgets()
{
	if (ToolBoxVBox)
	{
		ToolBoxVBox->ClearChildren();

		int32 PaletteCount = ActiveToolBarRows.Num();
		if (PaletteCount > 0)
		{
			for (int32 RowIdx = 0; RowIdx < PaletteCount; ++RowIdx)
			{
				const FEdModeToolbarRow& Row = ActiveToolBarRows[RowIdx];
				if (ensure(Row.ToolbarWidget.IsValid()))
				{
					TSharedRef<SWidget> PaletteWidget = Row.ToolbarWidget.ToSharedRef();

					ToolBoxVBox->AddSlot()
						.AutoHeight()
						.Padding(FMargin(2.0, 2.0))
						[
							SNew(SExpandableArea)
							.AreaTitle(Row.DisplayName)
						.AreaTitleFont(FAppStyle::Get().GetFontStyle("NormalFont"))
						.BorderImage(FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaHeader"))
						.BodyBorderImage(FAppStyle::Get().GetBrush("PaletteToolbar.ExpandableAreaBody"))
						.HeaderPadding(FMargin(4.f))
						.Padding(FMargin(4.0, 0.0))
						.BodyContent()
						[
							PaletteWidget
						]
						];

				}
			}
		}
	}

}

void FScriptableToolsEditorModeToolkit::RebuildModeToolkitBuilderPalettes()
{
	ToolkitBuilder->InitializeCategoryToolbar(true);
	
	const FScriptableToolsEditorModeManagerCommands& Commands = FScriptableToolsEditorModeManagerCommands::Get();
	const TSharedPtr<FUICommandList> CommandList = GetToolkitCommands();
	TMap<FName, TArray<TSharedPtr<FUICommandInfo>>> CommandLists = OwningEditorMode->GetModeCommands();

	TSharedPtr<FUICommandInfo> ActivePaletteOnLoad;
	TArray<FName> PaletteNames;
	GetToolPaletteNames(PaletteNames);
	const int PaletteCount = PaletteNames.Num();
	for (int32 PaletteId = 0; PaletteId < PaletteCount; ++PaletteId)
	{
		const FName PaletteName = PaletteNames[PaletteId];
		if (const TArray<FScriptableToolData>* PaletteTools = ActiveToolCategories.Find(PaletteName))
		{
			TArray<TSharedPtr<FUICommandInfo>> PaletteCommandList;
			for (const FScriptableToolData& Tool : *PaletteTools)
			{
				UScriptableInteractiveTool* ToolCDO = Cast<UScriptableInteractiveTool>(Tool.ToolClass->GetDefaultObject());
				if (!ToolCDO)
				{
					continue;
				}

				bool bFoundToolCommand = false;
				FString ToolIdentifier;
				Tool.ToolClass->GetClassPathName().ToString(ToolIdentifier);
				TSharedPtr<FUICommandInfo> ToolCommand = Commands.FindToolByName(ToolIdentifier, bFoundToolCommand);
				if (bFoundToolCommand)
				{
					PaletteCommandList.Add(ToolCommand);
				}
			}

			Algo::SortBy(PaletteCommandList, [](const TSharedPtr<FUICommandInfo>& Data)
			{
				return Data->GetCommandName().ToString();
			});

			bool bFoundLoadPaletteCommand = false;
			const FString LoadPaletteName = "LoadPalette" + PaletteName.ToString();
			TSharedPtr<FUICommandInfo> LoadPaletteCommand = Commands.FindToolByName(LoadPaletteName, bFoundLoadPaletteCommand);
			ActivePaletteOnLoad = LoadPaletteCommand;
			ensure(bFoundLoadPaletteCommand);
			ToolkitBuilder->AddPalette(
				MakeShareable( new FToolPalette(LoadPaletteCommand.ToSharedRef(), PaletteCommandList)) );
		}
	}

	if (ActivePaletteOnLoad)
	{
		ToolkitBuilder->SetActivePaletteOnLoad(ActivePaletteOnLoad.Get());
	}
	
	// This must be called prior to RefreshCategoryToolbarWidget since the visibility is not registered as a lambda.
	ToolkitBuilder->SetCategoryToolbarVisibility(PaletteCount > 1 ? EVisibility::Visible : EVisibility::Collapsed);
	ToolkitBuilder->RefreshCategoryToolbarWidget(true);
}

#undef LOCTEXT_NAMESPACE
