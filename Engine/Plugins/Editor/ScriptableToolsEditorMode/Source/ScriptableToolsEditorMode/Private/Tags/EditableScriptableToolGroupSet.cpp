// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tags/EditableScriptableToolGroupSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditableScriptableToolGroupSet)

UEditableScriptableToolGroupSet::UEditableScriptableToolGroupSet()
	: GroupsProperty(nullptr)
{
	if (!IsTemplate())
	{
		GroupsProperty = FindFProperty<FProperty>(GetClass(), TEXT("GroupSet"));
	}
}

void UEditableScriptableToolGroupSet::SetGroups(const FScriptableToolGroupSet::FGroupSet& InGroups)
{
	GroupSet.SetGroups(InGroups);
}

FScriptableToolGroupSet::FGroupSet& UEditableScriptableToolGroupSet::GetGroups()
{
	return GroupSet.GetGroups();
}

FString UEditableScriptableToolGroupSet::GetGroupSetExportText()
{
	GroupsPropertyAsString.Reset();

	if (GroupsProperty)
	{
		GroupsProperty->ExportTextItem_Direct(GroupsPropertyAsString, &GroupSet, &GroupSet, this, 0);
	}

	return GroupsPropertyAsString;
}
