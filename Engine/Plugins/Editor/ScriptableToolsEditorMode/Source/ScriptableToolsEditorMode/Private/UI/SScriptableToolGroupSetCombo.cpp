// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SScriptableToolGroupSetCombo.h"

#include "Tags/ScriptableToolGroupSet.h"
#include "Tags/ScriptableToolGroupTag.h"
#include "Tags/EditableScriptableToolGroupSet.h"
#include "PropertyHandle.h"
#include "Framework/Views/TableViewMetadata.h"
#include "UI/SScriptableToolGroupSetPicker.h"
#include "UI/SScriptableToolGroupTagChip.h"
#include "UObject/Package.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "ScriptableToolGroupSetCombo"

//------------------------------------------------------------------------------
// SScriptableToolGroupSetCombo
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SScriptableToolGroupSetCombo)
void SScriptableToolGroupSetCombo::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{

}

void SScriptableToolGroupSetCombo::Construct(const FArguments& InArgs)
{
	StructPropertyHandle = InArgs._StructPropertyHandle;
	StructPtr = InArgs._StructPtr;
	OnChanged = InArgs._OnChanged;

	HelperGroupSet.Reset(NewObject<UEditableScriptableToolGroupSet>(GetTransientPackage(), NAME_None, RF_Transient));

	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle() && !StructPtr)
	{
		StructPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SScriptableToolGroupSetCombo::RefreshListView));
	}

	RefreshListView();

	TWeakPtr<SScriptableToolGroupSetCombo> WeakSelf = StaticCastWeakPtr<SScriptableToolGroupSetCombo>(AsWeak());

	ActiveGroupTagsListView = SNew(SListView<UClass*>)
		.ListItemsSource(&ActiveGroupTags)
		.SelectionMode(ESelectionMode::None)
		.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
		.OnGenerateRow(this, &SScriptableToolGroupSetCombo::OnGenerateRow)
		.Visibility_Lambda([WeakSelf]()
			{
				if (const TSharedPtr<SScriptableToolGroupSetCombo> Self = WeakSelf.Pin())
				{
					return Self->ActiveGroupTags.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
				}
				return EVisibility::Collapsed;
			});

	ChildSlot
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Top)
		[
			SAssignNew(ComboButton, SComboButton)
			.HasDownArrow(true)
		.VAlign(VAlign_Top)
		.ContentPadding(0)
		.OnGetMenuContent(this, &SScriptableToolGroupSetCombo::OnGetMenuContent)
		.CollapseMenuOnParentFocus(true)
		.ButtonContent()
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Top)
			[
				SNew(SHorizontalBox)

				// Group Tag List
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Top)
				.FillWidth(1)
				[
					ActiveGroupTagsListView.ToSharedRef()
				]

				// Empty indicator
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.FillWidth(1)
				.Padding(FMargin(4, 2))
				[
					SNew(SBox)
					.HeightOverride(SScriptableToolGroupTagChip::ChipHeight)
					.VAlign(VAlign_Center)
					.Padding(0, 0, 8, 0)
					.Visibility_Lambda([WeakSelf]()
					{
						if (const TSharedPtr<SScriptableToolGroupSetCombo> Self = WeakSelf.Pin())
						{	
							return Self->ActiveGroupTags.Num() > 0 ? EVisibility::Collapsed : EVisibility::Visible;
						}
						return EVisibility::Collapsed;
					})
					[
						SNew(STextBlock)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
						.Text(LOCTEXT("ScriptableToolGroupSetCombo_Empty", "Select Tool Groups..."))
						.ToolTipText(LOCTEXT("ScriptableToolGroupSetCombo_EmptyTooltip", "No Tool Groups selected. Use dropdown to load tools into mode palette."))
					]
				]
			]
		]
		]
		];
	
}

TSharedRef<ITableRow> SScriptableToolGroupSetCombo::OnGenerateRow(UClass* InGroup, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<UClass*>, OwnerTable)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SimpleTableView.Row"))
		.Padding(FMargin(0, 2))		
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.0)
			[
				SNew(SScriptableToolGroupTagChip)
				.TagClass(InGroup)
				.Text(FText::FromString(Cast<UScriptableToolGroupTag>(InGroup->GetDefaultObject())->Name))
				.ToolTipText(FText::FromString(InGroup->GetClassPathName().ToString()))
				.OnClearPressed(this, &SScriptableToolGroupSetCombo::OnClearTagClicked, InGroup)
			]
		];
}

TSharedRef<SWidget> SScriptableToolGroupSetCombo::OnGetMenuContent()
{
	TagPicker = SNew(SScriptableToolGroupSetPicker)
		.StructPropertyHandle(StructPropertyHandle)
		.StructPtr(StructPtr)
		.OnChanged_Lambda([this]() {
			OnChanged.ExecuteIfBound();
		});

	ComboButton->SetMenuContentWidgetToFocus(TagPicker);

	return TagPicker.ToSharedRef();
}

FReply SScriptableToolGroupSetCombo::OnClearTagClicked(UClass* InGroup)
{
	HelperGroupSet->GetGroups().Remove(TSubclassOf<UScriptableToolGroupTag>(InGroup));

	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle() && !StructPtr)
	{
		// Set the property with a formatted string in order to propagate CDO changes to instances if necessary
		const FString OutString = HelperGroupSet->GetGroupSetExportText();
		StructPropertyHandle->SetValueFromFormattedString(OutString);
	}
	else if (StructPtr)
	{
		StructPtr->SetGroups(HelperGroupSet->GetGroups());
	}

	OnChanged.ExecuteIfBound();
	RefreshListView();

	return FReply::Handled();
}

void SScriptableToolGroupSetCombo::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!IsEnabled() && ComboButton->IsOpen())
	{
		ComboButton->SetIsOpen(false);
	}
}

void SScriptableToolGroupSetCombo::ForceUpdate()
{
	RefreshListView();
	if (TagPicker.IsValid())
	{
		TagPicker->ForceUpdate();
	}
}

void SScriptableToolGroupSetCombo::RefreshListView()
{
	ActiveGroupTags.Reset();

	// Add UClass* to our TagsToEdit list from the property handle
	void* StructPointer = nullptr;
	bool bValidData = false;
	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle() && !StructPtr)
	{
		if (StructPropertyHandle->GetValueData(StructPointer) == FPropertyAccess::Success && StructPointer)
		{
			bValidData = true;
		}
	}
	else if (StructPtr)
	{
		StructPointer = StructPtr;
		bValidData = true;
	}

	if(bValidData)
	{
		FScriptableToolGroupSet& GroupSet = *static_cast<FScriptableToolGroupSet*>(StructPointer);
		FScriptableToolGroupSet::FGroupSet Groups = GroupSet.GetGroups();

		HelperGroupSet->SetGroups(Groups);

		for (TSubclassOf<UScriptableToolGroupTag>& Element : Groups)
		{
			if (*Element)
			{
				ActiveGroupTags.AddUnique(*Element);
			}
		}
	}

	// Lexicographically sort Group tags.
	Algo::Sort(ActiveGroupTags, [](const UClass* LHS, const UClass* RHS)
		{
			const TObjectPtr<UScriptableToolGroupTag> LHSCDO = Cast<UScriptableToolGroupTag>(LHS->GetDefaultObject());
			const TObjectPtr<UScriptableToolGroupTag> RHSCDO = Cast<UScriptableToolGroupTag>(RHS->GetDefaultObject());

			check(IsValid(LHSCDO) && IsValid(RHSCDO));

			return LHSCDO->Name < RHSCDO->Name;
		});

	// Refresh the slate list
	if (ActiveGroupTagsListView.IsValid())
	{
		ActiveGroupTagsListView->SetItemsSource(&ActiveGroupTags);
		ActiveGroupTagsListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
