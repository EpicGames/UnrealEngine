// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SScriptableToolGroupSetPicker.h"

#include "Algo/ForEach.h"
#include "AssetRegistry/ARFilter.h"
#include "Kismet2/CompilerResultsLog.h"
#include "Tags/ScriptableToolGroupTag.h"
#include "Tags/EditableScriptableToolGroupSet.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "PropertyHandle.h"
#include "UI/SScriptableToolGroupTagChip.h"
#include "Textures/SlateIcon.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SCheckBox.h"

// Asset registry querying
#include "AssetRegistry/IAssetRegistry.h"
#include "Blueprint/BlueprintSupport.h"	// For FBlueprintTags::NativeParentClassPath
#include "AssetRegistry/AssetData.h"
#include "Engine/Blueprint.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Views/SListView.h"

#include "Kismet2/KismetEditorUtilities.h"
#include "AssetRegistry/AssetRegistryModule.h"

#define LOCTEXT_NAMESPACE "ScriptableToolGroupSetPicker"

//------------------------------------------------------------------------------
// SScriptableToolGroupSetPicker
//------------------------------------------------------------------------------

void SScriptableToolGroupSetPicker::Construct(const FArguments& InArgs)
{
	StructPropertyHandle = InArgs._StructPropertyHandle;
	StructPtr = InArgs._StructPtr;
	OnChanged = InArgs._OnChanged;

	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle() && !StructPtr)
	{
		StructPropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SScriptableToolGroupSetPicker::PopulateCheckedTags));
	}

	PopulateVisibleClasses();

	HelperGroupSet.Reset(NewObject<UEditableScriptableToolGroupSet>(GetTransientPackage(), NAME_None, RF_Transient));

	PopulateCheckedTags();

	OnSearchStringChanged(FText());

	ChildSlot
		[
			GetChildWidget()
		];	
}

TSharedRef<SWidget> SScriptableToolGroupSetPicker::GetChildWidget()
{
	const TSharedRef<SWidget> MenuContent = SNew(SBox)
		[
			SNew(SVerticalBox)

			// Search box
		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Top)
		[
			SAssignNew(GroupSearchBox, SSearchBox)
			.HintText(LOCTEXT("ScriptableToolGroupSetPicker_SearchBoxHint", "Search Tool Group Tags"))
		.OnTextChanged(this, &SScriptableToolGroupSetPicker::OnSearchStringChanged)
		]

	// List of tags
	+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(VisibleGroupTagsListView, SListView<UClass*>)
			.ListItemsSource(&VisibleGroupTags)
		.OnGenerateRow(this, &SScriptableToolGroupSetPicker::OnGenerateRow)
		.SelectionMode(ESelectionMode::None)
		.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
		]
		];



	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection*/false, nullptr);

	MenuBuilder.BeginSection(FName(), LOCTEXT("SectionGroupSet", "Tool Group Tags"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ScriptableToolGroupSetPicker_CreateNewTag", "Create New Tag"),
		FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus"),
		FUIAction(FExecuteAction::CreateRaw(this, &SScriptableToolGroupSetPicker::OnCreateNewTag))
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ScriptableToolGroupSetPicker_ClearAllTags", "Clear All Tags"), FText::GetEmpty(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.X"),
		FUIAction(FExecuteAction::CreateRaw(this, &SScriptableToolGroupSetPicker::OnUncheckAllTags))
	);

	MenuBuilder.AddSeparator();

	MenuBuilder.AddWidget(MenuContent, FText::GetEmpty(), true);

	MenuBuilder.EndSection();



	return MenuBuilder.MakeWidget();
}

TSharedRef<ITableRow> SScriptableToolGroupSetPicker::OnGenerateRow(UClass* InTag, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (const TObjectPtr<UScriptableToolGroupTag> SubclassCDO = Cast<UScriptableToolGroupTag>(InTag->GetDefaultObject()))
	{
		return SNew(STableRow<UClass*>, OwnerTable)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged_Lambda([this, InTag](ECheckBoxState NewState)
					{
						if (NewState == ECheckBoxState::Checked)
						{
							OnTagChecked(InTag);
						}
						else if (NewState == ECheckBoxState::Unchecked)
						{
							OnTagUnchecked(InTag);
						}
					})
			.IsChecked_Lambda([this, InTag]()
				{
					return IsTagChecked(InTag);
				})
						.ToolTipText(FText::FromString(InTag->GetClassPathName().ToString()))
					.Content()
					[
						SNew(STextBlock)
						.Text(FText::FromString(SubclassCDO->Name))
					]
			];
	}

	return SNew(STableRow<UClass*>, OwnerTable);
}

void SScriptableToolGroupSetPicker::OnSearchStringChanged(const FText& NewString)
{
	SearchString = NewString.ToString();
	RefreshListView();
}

void SScriptableToolGroupSetPicker::RefreshListView()
{
	VisibleGroupTags.Empty();

	// Add all tags matching the search string.
	for (UClass* Subclass : AllGroupTags)
	{
		if (const TObjectPtr<UScriptableToolGroupTag> SubclassCDO = Cast<UScriptableToolGroupTag>(Subclass->GetDefaultObject()))
		{
			if (SearchString.IsEmpty() || SubclassCDO->Name.Contains(SearchString))
			{
				VisibleGroupTags.AddUnique(Subclass);
			}
		}
	}

	// Lexicographically sort group tags.
	Algo::Sort(VisibleGroupTags, [](const UClass* LHS, const UClass* RHS)
		{
			const TObjectPtr<UScriptableToolGroupTag> LHSCDO = Cast<UScriptableToolGroupTag>(LHS->GetDefaultObject());
			const TObjectPtr<UScriptableToolGroupTag> RHSCDO = Cast<UScriptableToolGroupTag>(RHS->GetDefaultObject());

			check(IsValid(LHSCDO) && IsValid(RHSCDO));

			return LHSCDO->Name < RHSCDO->Name;
		});

	// Refresh the slate list.
	if (VisibleGroupTagsListView.IsValid())
	{
		VisibleGroupTagsListView->SetItemsSource(&VisibleGroupTags);
		VisibleGroupTagsListView->RequestListRefresh();
	}
}

void SScriptableToolGroupSetPicker::PopulateVisibleClasses()
{
	const IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	const FTopLevelAssetPath BaseAssetPath = UScriptableToolGroupTag::StaticClass()->GetClassPathName();

	AllGroupTags.Reset();

	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	Filter.TagsAndValues.Add(FBlueprintTags::NativeParentClassPath);

	AssetRegistry.EnumerateAssets(Filter, [this, BaseAssetPath](const FAssetData& Data)
		{
			if (Data.TagsAndValues.FindTag(FBlueprintTags::NativeParentClassPath).AsExportPath().ToTopLevelAssetPath() == BaseAssetPath)
			{
				if (const UBlueprint* AssetAsBlueprint = Cast<UBlueprint>(Data.GetAsset()))
				{
					if (const TSubclassOf<UScriptableToolGroupTag> GeneratedClassAsGroupClass = *AssetAsBlueprint->GeneratedClass)
					{
						AllGroupTags.Add(GeneratedClassAsGroupClass);
					}
				}
			}
			return true;	// Returning false will halt the enumeration
		});
}

void SScriptableToolGroupSetPicker::ForceUpdate()
{
	PopulateCheckedTags();
}

void SScriptableToolGroupSetPicker::PopulateCheckedTags()
{
	// Reset checked tag map
	for (UClass* Group : AllGroupTags)
	{
		CheckedTags.FindOrAdd(Group) = false;
	}

	// Access group set array
	void* StructPointer = nullptr;
	bool bValidData = false;
	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle() && !StructPtr)
	{
		if (StructPropertyHandle->GetValueData(StructPointer) == FPropertyAccess::Success && StructPointer)
		{
			bValidData = true;
		}
	}
	else if (StructPtr)
	{
		StructPointer = StructPtr;
		bValidData = true;
	}

	if (bValidData)
	{
		FScriptableToolGroupSet& GroupSet = *static_cast<FScriptableToolGroupSet*>(StructPointer);
		FScriptableToolGroupSet::FGroupSet Groups = GroupSet.GetGroups();

		HelperGroupSet->SetGroups(Groups);

		// Bring checked tag map to parity with current group set
		for (TSubclassOf<UScriptableToolGroupTag>& Element : Groups)
		{
			if (CheckedTags.Contains(Element))
			{
				CheckedTags[Element] = true;
			}
		}
	}
}

ECheckBoxState SScriptableToolGroupSetPicker::IsTagChecked(UClass* InTag)
{
	// This will add InTag to the map if not in it with a default value of false.
	return CheckedTags.FindOrAdd(InTag) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SScriptableToolGroupSetPicker::OnTagChecked(UClass* InTag)
{
	CheckedTags.FindOrAdd(InTag) = true;
	HelperGroupSet->GetGroups().FindOrAdd(InTag);	// Expected value defaults to true when adding a group.
	FlushHelperGroupSet();
}

void SScriptableToolGroupSetPicker::OnTagUnchecked(UClass* InTag)
{
	CheckedTags.FindOrAdd(InTag) = false;
	HelperGroupSet->GetGroups().Remove(InTag);
	FlushHelperGroupSet();
}

void SScriptableToolGroupSetPicker::OnUncheckAllTags()
{
	Algo::ForEach(AllGroupTags, [this](UClass* Group) { CheckedTags.FindOrAdd(Group) = false; });
	HelperGroupSet->GetGroups().Empty();
	FlushHelperGroupSet();
}

void SScriptableToolGroupSetPicker::OnCreateNewTag()
{

	UBlueprint* NewBlueprint = FKismetEditorUtilities::CreateBlueprintFromClass(
		LOCTEXT("ScriptableToolGroupSetPicker_CreateTagAssetDialog","Create a new ScriptableTool group tag"),
		UScriptableToolGroupTag::StaticClass(),
		TEXT("NewTag"));

	if (NewBlueprint)
	{
		// Editing the BluePrint
		UScriptableToolGroupTag* NewBlueprintDefaultValues = NewBlueprint->GeneratedClass->GetDefaultObject<UScriptableToolGroupTag>();
		NewBlueprintDefaultValues->Name = FString(NewBlueprintDefaultValues->GetName()).LeftChop(2).RightChop(9); // Taking off the "Default__<Name>_C" bits

		// Compile the changes
		FCompilerResultsLog LogResults;
		FKismetEditorUtilities::CompileBlueprint(NewBlueprint, EBlueprintCompileOptions::None, &LogResults);

		// Notify the asset registry
		FAssetRegistryModule::AssetCreated(NewBlueprint);
	}
}

void SScriptableToolGroupSetPicker::FlushHelperGroupSet() const
{
	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle() && !StructPtr)
	{
		// Set the property with a formatted string in order to propagate CDO changes to instances if necessary
		const FString OutString = HelperGroupSet->GetGroupSetExportText();
		StructPropertyHandle->SetValueFromFormattedString(OutString);
	}
	else if (StructPtr)
	{
		StructPtr->SetGroups(HelperGroupSet->GetGroups());
	}

	OnChanged.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE
