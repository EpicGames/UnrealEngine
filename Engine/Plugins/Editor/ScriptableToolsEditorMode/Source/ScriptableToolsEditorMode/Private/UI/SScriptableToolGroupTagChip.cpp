// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SScriptableToolGroupTagChip.h"

#include "Styling/StyleColors.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "ScriptableToolGroupTagChip"

//------------------------------------------------------------------------------
// SScriptableToolGroupTagChip
//------------------------------------------------------------------------------

SLATE_IMPLEMENT_WIDGET(SScriptableToolGroupTagChip)
void SScriptableToolGroupTagChip::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "Text", TextAttribute, EInvalidateWidgetReason::Layout);
	SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION_WITH_NAME(AttributeInitializer, "ToolTipText", ToolTipTextAttribute, EInvalidateWidgetReason::Layout);
}

SScriptableToolGroupTagChip::SScriptableToolGroupTagChip()
	: ToolTipTextAttribute(*this)
	, TextAttribute(*this)
{}

void SScriptableToolGroupTagChip::Construct(const FArguments& InArgs)
{
	ToolTipTextAttribute.Assign(*this, InArgs._ToolTipText);
	TextAttribute.Assign(*this, InArgs._Text);
	OnClearPressed = InArgs._OnClearPressed;
	TagClass = InArgs._TagClass;

	TWeakPtr<SScriptableToolGroupTagChip> WeakSelf = StaticCastWeakPtr<SScriptableToolGroupTagChip>(AsWeak());

	ChildSlot
		[
			SNew(SBox)
			.HeightOverride(ChipHeight)		
		[
			SNew(SButton)			
			.VAlign(VAlign_Center)
		.HAlign(HAlign_Fill)
		.OnClicked_Lambda([WeakSelf]()
			{
				return FReply::Unhandled();
			})
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.Padding(10.f, 0.f)
				[
					SNew(STextBlock)
					.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.Text_Lambda([WeakSelf]()
					{
						const TSharedPtr<SScriptableToolGroupTagChip> Self = WeakSelf.Pin();
						return Self.IsValid() ? Self->TextAttribute.Get() : FText::GetEmpty();
					})
				.ToolTipText_Lambda([WeakSelf]()
					{
						const TSharedPtr<SScriptableToolGroupTagChip> Self = WeakSelf.Pin();
						return Self.IsValid() ? Self->ToolTipTextAttribute.Get() : FText::GetEmpty();
					})
				]

			// Clear Button
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				.Padding(0.f)
				[
					SAssignNew(ClearButton, SButton)
					.Visibility_Lambda([WeakSelf]()
						{
							const TSharedPtr<SScriptableToolGroupTagChip> Self = WeakSelf.Pin();
							return Self.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
						})
				.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.ToolTipText(LOCTEXT("ClearTag", "Clear Tag"))
							.ContentPadding(0)
							.OnClicked_Lambda([WeakSelf]()
								{
									//UE_LOGF(LogScriptableToolEditor, Log, "Clear button clicked!")
										const TSharedPtr<SScriptableToolGroupTagChip> Self = WeakSelf.Pin();
									if (Self.IsValid() && Self->OnClearPressed.IsBound())
									{
										return Self->OnClearPressed.Execute();
									}
									return FReply::Unhandled();
								})
							[
								SNew(SImage)
								.ColorAndOpacity_Lambda([WeakSelf]()
									{
										const TSharedPtr<SScriptableToolGroupTagChip> Self = WeakSelf.Pin();
										if (Self.IsValid() && Self->ClearButton.IsValid())
										{
											return Self->ClearButton->IsHovered() ? FStyleColors::White : FStyleColors::Foreground;
										}
										return FStyleColors::Foreground;
									})
									.Image(FAppStyle::GetBrush("Icons.X"))
										.DesiredSizeOverride(FVector2D(12.f, 12.f))
							]
				]
		]
		]
		];
}

#undef LOCTEXT_NAMESPACE
