// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/ScriptableToolGroupSetCustomization.h"

#include "DetailWidgetRow.h"
#include "Tags/EditableScriptableToolGroupSet.h" // IWYU pragma: keep
#include "PropertyHandle.h"
#include "UI/SScriptableToolGroupSetCombo.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FScriptableToolTrackGroupSetCustomization"

TSharedRef<IPropertyTypeCustomization> FScriptableToolGroupSetCustomization::MakeInstance()
{
	return MakeShareable(new FScriptableToolGroupSetCustomization);
}

void FScriptableToolGroupSetCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
	.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			[
				SNew(SScriptableToolGroupSetCombo)
				.StructPropertyHandle(StructPropertyHandle)
			]
		];
}

void FScriptableToolGroupSetCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{}

#undef LOCTEXT_NAMESPACE
