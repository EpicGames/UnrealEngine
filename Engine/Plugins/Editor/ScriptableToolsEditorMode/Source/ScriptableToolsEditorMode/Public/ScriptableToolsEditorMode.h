// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Tools/LegacyEdModeWidgetHelpers.h"

#include "InputState.h"
#include "InteractiveToolManager.h"
#include "Tools/EdModeInteractiveToolsContext.h"

#include "ScriptableToolsEditorMode.generated.h"

#define UE_API SCRIPTABLETOOLSEDITORMODE_API

class FUICommandList;
class FLevelObjectsObserver;
class UModelingSceneSnappingManager;
class UModelingSelectionInteraction;
class UGeometrySelectionManager;
class UScriptableToolSet;
class UBlueprint;
class UScriptableToolContextObject;

UCLASS(MinimalAPI, Transient)
class UScriptableToolsEditorMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()
public:
	UE_API const static FEditorModeID EM_ScriptableToolsEditorModeId;

	UE_API UScriptableToolsEditorMode();
	UE_API UScriptableToolsEditorMode(FVTableHelper& Helper);
	UE_API ~UScriptableToolsEditorMode();
	////////////////
	// UEdMode interface
	////////////////

	UE_API virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;

	UE_API virtual void ActorSelectionChangeNotify() override;

	UE_API virtual bool ShouldDrawWidget() const override;
	UE_API virtual bool ProcessEditDelete() override;
	UE_API virtual bool ProcessEditCut() override;

	UE_API virtual bool CanAutoSave() const override;

	UE_API virtual bool ComputeBoundingBoxForViewportFocus(AActor* Actor, UPrimitiveComponent* PrimitiveComponent, FBox& InOutBox) const override;

	UE_API virtual bool GetPivotForOrbit(FVector& OutPivot) const override;
	
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }

	/*
	 * focus events
	 */

	// called when we "start" this editor mode (ie switch to this tab)
	UE_API virtual void Enter() override;

	// called when we "end" this editor mode (ie switch to another tab)
	UE_API virtual void Exit() override;

	UE_API virtual bool ShouldToolStartBeAllowed(const FString& ToolIdentifier) const override;

	//////////////////
	// End of UEdMode interface
	//////////////////

	

protected:
	UE_API virtual void BindCommands() override;
	UE_API virtual void CreateToolkit() override;
	UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	
	UE_API virtual void OnToolPostBuild(UInteractiveToolManager* InToolManager, EToolSide InSide, UInteractiveTool* InBuiltTool, UInteractiveToolBuilder* InToolBuilder, const FToolBuilderState& ToolState);

	UE_API void OnToolsContextRender(IToolsContextRenderAPI* RenderAPI);

	UE_API void AcceptActiveToolActionOrTool();
	UE_API void CancelActiveToolActionOrTool();

	UE_API void ConfigureRealTimeViewportsOverride(bool bEnable);

	UE_API void OnBlueprintPreCompile(UBlueprint* Blueprint);
	FDelegateHandle BlueprintPreCompileHandle;

	UE_DEPRECATED(5.6, "Deprecated in favor of OnBlueprintPreCompile")
	void OnBlueprintCompiled() {}

	UE_DEPRECATED(5.6, "Deprecated in favor of OnBlueprintPreCompile")
	FDelegateHandle BlueprintCompiledHandle;

	UE_API void InitializeModeContexts();

	UE_API void RebuildScriptableToolSet();

	void SetFocusInViewport();

private:

	TArray<TWeakObjectPtr<UScriptableToolContextObject>> ContextsToUpdateOnToolEnd;
	TArray<TWeakObjectPtr<UScriptableToolContextObject>> ContextsToShutdown;

	bool bRebuildScriptableToolSetOnTick = false;

protected:
	UPROPERTY()
	TObjectPtr<UScriptableToolSet> ScriptableTools;
public:
	virtual UScriptableToolSet* GetActiveScriptableTools() { return ScriptableTools; }

};

#undef UE_API
