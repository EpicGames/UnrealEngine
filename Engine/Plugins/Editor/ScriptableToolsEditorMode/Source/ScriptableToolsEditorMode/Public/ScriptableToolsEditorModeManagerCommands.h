// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

/**
 * TInteractiveToolCommands implementation for Modeling Mode Tools
 */
class SCRIPTABLETOOLSEDITORMODE_API FScriptableToolsEditorModeManagerCommands : public TCommands<FScriptableToolsEditorModeManagerCommands>
{
public:
	FScriptableToolsEditorModeManagerCommands();

protected:
	struct FStartToolCommand
	{
		FString ToolUIName;
		TSharedPtr<FUICommandInfo> ToolCommand;
	};
	TArray<FStartToolCommand> RegisteredTools;		// Tool start-commands listed below are stored in this list

public:
	/**
	 * Find Tool start-command below by registered name (tool icon name in Mode palette)
	 */
	TSharedPtr<FUICommandInfo> FindToolByName(const FString& Name, bool& bFound) const;

	TSharedPtr<FUICommandInfo> RegisterCommand(
		FName InCommandName,
		const FText& InCommandLabel,
		const FText& InCommandTooltip,
		const FSlateIcon& InIcon,
		const EUserInterfaceActionType InUserInterfaceType,
		const FInputChord& InDefaultChord);

	void NotifyCommandsChanged() const;
	
	//
	// Accept/Cancel/Complete commands are used to end the active Tool via ToolManager
	//

	TSharedPtr<FUICommandInfo> AcceptActiveTool;
	TSharedPtr<FUICommandInfo> CancelActiveTool;
	TSharedPtr<FUICommandInfo> CompleteActiveTool;

	TSharedPtr<FUICommandInfo> AcceptOrCompleteActiveTool;
	TSharedPtr<FUICommandInfo> CancelOrCompleteActiveTool;

	/**
	 * Initialize above commands
	 */
	virtual void RegisterCommands() override;
};

