// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Tags/ScriptableToolGroupSet.h"
#include "Templates/SharedPointer.h"

#include "ScriptableToolsEditorModeSettings.generated.h"

#define UE_API SCRIPTABLETOOLSEDITORMODE_API


UCLASS(MinimalAPI, config = Editor)
class UScriptableToolsModeCustomizationSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	// UDeveloperSettings overrides

	virtual FName GetContainerName() const { return FName("Project"); }
	virtual FName GetCategoryName() const { return FName("Plugins"); }
	virtual FName GetSectionName() const { return FName("ScriptableTools"); }

	UE_API virtual FText GetSectionText() const override;
	UE_API virtual FText GetSectionDescription() const override;

public:

	UPROPERTY(config, EditAnywhere, Category = "Scriptable Tools Mode|Tool Registration", meta = (EditCondition = "!bRegisterAllTools"))
	FScriptableToolGroupSet ToolRegistrationFilters;

	/** Toggle between the Legacy Scriptable Tools Palette and the new UI (requires exiting and re-entering the Mode) */
	UPROPERTY(config, EditAnywhere, Category = "Scriptable Tools Mode|UI Customization")
	bool bUseLegacyPalette = false;

	/** If true, Tool buttons will always be shown when in a Tool. By default they will be hidden. */
	UPROPERTY(config, EditAnywhere, Category = "Scriptable Tools Mode|UI Customization")
	bool bAlwaysShowToolButtons = true;

	bool RegisterAllTools() const {	return ToolRegistrationFilters.GetGroups().IsEmpty(); }

};

#undef UE_API
