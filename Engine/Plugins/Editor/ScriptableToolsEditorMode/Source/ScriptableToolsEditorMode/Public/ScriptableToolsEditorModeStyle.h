// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

#define UE_API SCRIPTABLETOOLSEDITORMODE_API

class FScriptableToolsEditorModeStyle
{
public:
	static UE_API void Initialize();

	static UE_API void Shutdown();

	static UE_API TSharedPtr< class ISlateStyle > Get();

	static UE_API FName GetStyleSetName();

	// use to access icons defined by the style set by name, eg GetBrush("BrushFalloffIcons.Smooth")
	static UE_API const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = NULL);


	// Try to find a png or svg icon at FileNameWithPath and register it in the style set with the given StyleIconIdentifier,
	// ie so that later it can be found FSlateIcon(FScriptableToolsEditorModeStyle::Get()->GetStyleSetName(), StyleIconIdentifier).
	// This is intended to be used for (eg) custom icons defined in Scriptable Tool Blueprints, that are not known at compile time.
	// 
	// Will print error and return false if image is not found, or not png/svg format.
	// 
	// FileNameWithPath must be the full on-disk path to the icon, which is constructed in code from a relative path
	// the user enters as (eg) a property/etc. To help with debugging, this path can be provided as ExternalRelativePath,
	// and it will be printed out instead of the absolute path as part of the log error message.
	//
	UE_DEPRECATED(5.7, "Use RegisterIconTexture(s) instead.")
	static UE_API bool TryRegisterCustomIcon(FName StyleIconIdentifier, FString FileNameWithPath, FString ExternalRelativePath);

	// Register tool icons from a UTexture2D
	using FIconTextureMap = TMap<FName, TObjectPtr<UTexture2D>>;
	static UE_API bool RegisterIconTexture(FName StyleIdentifier, TObjectPtr<UTexture2D> InTexture);
	static UE_API void RegisterIconTextures(const FIconTextureMap& InIcons);


private:
	static UE_API FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	static UE_API TSharedPtr< class FSlateStyleSet > StyleSet;

	static TMap<FName, FSlateBrush*> IconTextureBrushes;
};

#undef UE_API
