// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/BaseToolkit.h"
#include "ScriptableToolsEditorMode.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "StatusBarSubsystem.h"

#define UE_API SCRIPTABLETOOLSEDITORMODE_API


class IDetailsView;
class SButton;
class STextBlock;
class UBlueprint;
class UBaseScriptableToolBuilder;

class FScriptableToolsEditorModeToolkit : public FModeToolkit
{
public:

	UE_API FScriptableToolsEditorModeToolkit();
	UE_API ~FScriptableToolsEditorModeToolkit();
	
	/** FModeToolkit interface */
	UE_API virtual void Init(const TSharedPtr<IToolkitHost>& InitToolkitHost, TWeakObjectPtr<UEdMode> InOwningMode) override;

	/** IToolkit interface */
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	virtual TSharedPtr<class SWidget> GetInlineContent() const override { return ToolkitWidget; }

	// initialize toolkit widgets that need to wait until mode is initialized/entered
	UE_API virtual void InitializeAfterModeSetup();

	// set/clear notification message area
	UE_API virtual void PostNotification(const FText& Message);
	UE_API virtual void ClearNotification();

	// set/clear warning message area
	UE_API virtual void PostWarning(const FText& Message);
	UE_API virtual void ClearWarning();

	// Async Tool Loading
	UE_API void StartAsyncToolLoading();
	UE_API void SetAsyncProgress(float PercentLoaded);
	UE_API void EndAsyncToolLoading();
	UE_API bool AreToolsLoading() const;
	UE_API TOptional<float> GetToolPercentLoaded() const;

	/** Returns the Mode specific tabs in the mode toolbar **/ 
	UE_API virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const override;
	UE_API virtual FText GetToolPaletteDisplayName(FName PaletteName) const override; 
	UE_API virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder) override;
	UE_API virtual void OnToolPaletteChanged(FName PaletteName) override;
	virtual bool HasIntegratedToolPalettes() const override { return false; }
	virtual bool HasExclusiveToolPalettes() const override { return false; }

	virtual FText GetActiveToolDisplayName() const override { return ActiveToolName; }
	virtual FText GetActiveToolMessage() const override { return ActiveToolMessage; }

	UE_API virtual void EnableShowRealtimeWarning(bool bEnable);

	UE_API virtual void OnToolStarted(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;
	UE_API virtual void OnToolEnded(UInteractiveToolManager* Manager, UInteractiveTool* Tool) override;

	UE_API virtual void CustomizeModeDetailsViewArgs(FDetailsViewArgs& ArgsInOut) override;

	UE_API void OnActiveViewportChanged(TSharedPtr<IAssetViewport>, TSharedPtr<IAssetViewport> );

	UE_API virtual void InvokeUI() override;

	UE_API virtual void ForceToolPaletteRebuild();

	UE_API void GetActiveToolPaletteNames(TArray<FName>& OutPaletteNames);

	UE_API void SetShowToolTagFilterBar(bool bShowToolTagFilterBarIn);


protected:

	/** FModeToolkit interface */
	UE_API virtual void RebuildModeToolBar() override;
	UE_API virtual bool ShouldShowModeToolbar() const override;

	UE_API void RebuildModeToolPaletteWidgets();
	UE_API void RebuildModeToolkitBuilderPalettes();


private:
	UE_API const static TArray<FName> PaletteNames_Standard;

	FText ActiveToolName;
	FText ActiveToolMessage;
	FStatusBarMessageHandle ActiveToolMessageHandle;
	const FSlateBrush* ActiveToolIcon = nullptr;

	TSharedPtr<SWidget> ToolkitWidget;
	UE_API void UpdateActiveToolProperties();
	UE_API void InvalidateCachedDetailPanelState(UObject* ChangedObject);
	
	UE_API void RegisterPalettes();
	FDelegateHandle ActivePaletteChangedHandle;

	TSharedPtr<SWidget> ViewportOverlayWidget;

	TSharedPtr<STextBlock> ModeWarningArea;
	TSharedPtr<STextBlock> ModeHeaderArea;
	TSharedPtr<STextBlock> ToolWarningArea;
	TSharedPtr<SButton> AcceptButton;
	TSharedPtr<SButton> CancelButton;
	TSharedPtr<SButton> CompletedButton;

	// Palette
	bool bAsyncLoadInProgress = false;
	float AsyncLoadProgress;

	TSharedPtr<SVerticalBox> ToolBoxVBox;
	FDelegateHandle SettingsUpdateHandle;

	TSharedPtr<SWidget> ToolPaletteHeader;
	TSharedPtr<SWidget> ToolPaletteTagPanel;
	TSharedPtr<SWidget> ToolPaletteLoadBar;
	TSharedPtr<SVerticalBox> ToolDetailsHeader;

	bool bShowRealtimeWarning = false;
	UE_API void UpdateShowWarnings();

	struct FScriptableToolData
	{
		FText Category;
		UClass* ToolClass = nullptr;
		UBaseScriptableToolBuilder* Builder = nullptr;
	};
	TMap<FName, TArray<FScriptableToolData>> ActiveToolCategories;
	UE_API void UpdateActiveToolCategories();

	bool bFirstInitializeAfterModeSetup = true;

	bool bShowActiveSelectionActions = false;

	bool bShowToolTagFilterBar = true;

	// custom accept/cancel/complete handlers
};

#undef UE_API
