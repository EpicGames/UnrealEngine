// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tags/ScriptableToolGroupSet.h"

#include "EditableScriptableToolGroupSet.generated.h"

UCLASS(Transient, MinimalAPI)
class UEditableScriptableToolGroupSet : public UObject
{
public:
	GENERATED_BODY()

	UEditableScriptableToolGroupSet();

	void SetGroups(const FScriptableToolGroupSet::FGroupSet& InGroups);
	FScriptableToolGroupSet::FGroupSet& GetGroups();

	FString GetGroupSetExportText();

private:
	UPROPERTY()
	FScriptableToolGroupSet GroupSet;

	FProperty* GroupsProperty;
	FString GroupsPropertyAsString;
};