// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"
#include "Tags/EditableScriptableToolGroupSet.h"

template <typename ItemType> class SListView;

class IPropertyHandle;
class SMenuAnchor;
class ITableRow;
class STableViewBase;
class SScriptableToolGroupSetPicker;
class SComboButton;
struct FScriptableToolGroupSet;

/**
 * Widget for editing a group set.
 */
class SScriptableToolGroupSetCombo : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SScriptableToolGroupSetCombo, SCompoundWidget)

public:

	DECLARE_DELEGATE(FOnChanged);

	SLATE_BEGIN_ARGS(SScriptableToolGroupSetCombo)
		: _StructPropertyHandle(nullptr), _StructPtr(nullptr)
	{}
	SLATE_EVENT(FOnChanged, OnChanged)
	// Used for writing changes to the group set being edited. 
	SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, StructPropertyHandle)
	// An alternate to the Property Handle, allowing use directly with a pointer to a tag set.
	SLATE_ARGUMENT(FScriptableToolGroupSet*, StructPtr)
	
	SLATE_END_ARGS();

	SCRIPTABLETOOLSEDITORMODE_API void Construct(const FArguments& InArgs);

	/** Used to update internal state based on underlying data by external clients */
	void ForceUpdate();

protected:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:
	/** Returns a table row with an SScriptableToolGroupTagChip. */
	TSharedRef<ITableRow> OnGenerateRow(UClass* InGroup, const TSharedRef<STableViewBase>& OwnerTable);

	/** Instantiates the tag picker and sets it as the widget to focus for the ComboButton. */
	TSharedRef<SWidget> OnGetMenuContent();

	/** Bound to/called via SScriptableToolGroupTagChip::OnClearPressed. Removes TagToClear from group set. */
	FReply OnClearTagClicked(UClass* InGroup);

	/** Populates ActiveGroupTags with the groups currently active on the group set. */
	void RefreshListView();

	/** The set of group tags to display, based on the group tags present in the group set we are editing. */
	TArray<UClass*> ActiveGroupTags;
	TSharedPtr<SListView<UClass*>> ActiveGroupTagsListView;

	/** Widgets we retain ownership of and refer to in a named manner. */
	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SScriptableToolGroupSetPicker> TagPicker;

	/** Property handle to an FScriptableToolGroupSet, used for accessing the source group set. */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** A direct pointer to a group Set, an alternate pathway to the PropertyHandle */
	FScriptableToolGroupSet* StructPtr;

	/** A helper class which is used for propagating changes to the source group set. */
	TStrongObjectPtr<UEditableScriptableToolGroupSet> HelperGroupSet;

	FOnChanged OnChanged;
};
