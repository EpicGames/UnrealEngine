// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/SCompoundWidget.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"

class ITableRow;
class SSearchBox;
class STableViewBase;
enum class ECheckBoxState : uint8;
template <typename ItemType> class SListView;

class UEditableScriptableToolGroupSet;
class IPropertyHandle;
class SComboButton;
struct FScriptableToolGroupSet;

/**
 * Widget allowing user to edit the group tags in a given group set.
 */
class SScriptableToolGroupSetPicker : public SCompoundWidget
{
public:
	DECLARE_DELEGATE(FOnChanged);

	SLATE_BEGIN_ARGS(SScriptableToolGroupSetPicker)
		: _StructPropertyHandle(nullptr)
	{}
	SLATE_EVENT(FOnChanged, OnChanged)
	// Used for writing changes to the group set being edited. 
	SLATE_ARGUMENT(TSharedPtr<IPropertyHandle>, StructPropertyHandle)
	// An alternate to the Property Handle, allowing use directly with a pointer to a tag set.
	SLATE_ARGUMENT(FScriptableToolGroupSet*, StructPtr)
	SLATE_END_ARGS()

	/** Construct the actual widget */
		SCRIPTABLETOOLSEDITORMODE_API void Construct(const FArguments& InArgs);

	/** Used to update internal state based on underlying data by external clients */
	void ForceUpdate();

private:
	TSharedRef<SWidget> GetChildWidget();
	TSharedRef<ITableRow> OnGenerateRow(UClass* InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Members for tracking and using search text. */
	void OnSearchStringChanged(const FText& NewString);
	TSharedPtr<SSearchBox> GroupSearchBox;
	FString SearchString;

	/** Populate VisibleGroupTags with the correct subset of AllGroupTags given our current search text. */
	void RefreshListView();

	/** The set of group tags to display, based on our current search text. */
	TArray<UClass*> VisibleGroupTags;
	TSharedPtr<SListView<UClass*>> VisibleGroupTagsListView;

	/** Queries the asset registry to get all known group tags. */
	void PopulateVisibleClasses();

	/** The set of group tags which can be displayed. */
	TArray<UClass*> AllGroupTags;

	/** Functions for querying the group set for information about a particular tag. */
	ECheckBoxState IsTagChecked(UClass* InTag);
	void PopulateCheckedTags();
	TMap<UClass*, bool> CheckedTags;

	/** Functions for adding/removing tags to/from the group set. */
	void OnTagChecked(UClass* NodeChecked);
	void OnTagUnchecked(UClass* NodeUnchecked);
	void OnUncheckAllTags();
	void OnCreateNewTag();

	/** Brings the source group set to parity with HelperGroupSet. Generally called immediately after modifying HelperGroupSet. */
	void FlushHelperGroupSet() const;

	/** Property handle to an FScriptableToolGroupSet, used for accessing the source group set. */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** A direct pointer to a Group Set, an alternate pathway to the PropertyHandle */
	FScriptableToolGroupSet* StructPtr;

	/** A helper class which is used for propagating changes to the source group set. */
	TStrongObjectPtr<UEditableScriptableToolGroupSet> HelperGroupSet;

	FOnChanged OnChanged;
};
