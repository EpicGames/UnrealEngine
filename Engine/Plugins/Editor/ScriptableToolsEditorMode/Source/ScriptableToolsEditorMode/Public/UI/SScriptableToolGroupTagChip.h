// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class SButton;

/**
 * Widget for displaying a single group tag.
 */
class SScriptableToolGroupTagChip : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SScriptableToolGroupTagChip, SCompoundWidget)

public:

	DECLARE_DELEGATE_RetVal(FReply, FOnClearPressed);

		SLATE_BEGIN_ARGS(SScriptableToolGroupTagChip)
	{}
	// The group subclass associated with this chip.
	SLATE_ARGUMENT(UClass*, TagClass)

		// Callback for when clear tag button is pressed.
		SLATE_EVENT(FOnClearPressed, OnClearPressed)

		// Tooltip to display.
		SLATE_ATTRIBUTE(FText, ToolTipText)

		// Text to display.
		SLATE_ATTRIBUTE(FText, Text)
		SLATE_END_ARGS();

	SCRIPTABLETOOLSEDITORMODE_API SScriptableToolGroupTagChip();
	SCRIPTABLETOOLSEDITORMODE_API void Construct(const FArguments& InArgs);

	/** This is public because it is used in SScriptableToolGroupSetCombo and SScriptableToolGroupSetPicker. */
	inline static constexpr float ChipHeight = 25.f;

private:
	TSlateAttribute<FText> ToolTipTextAttribute;
	TSlateAttribute<FText> TextAttribute;

	FOnClearPressed OnClearPressed;

	TSharedPtr<SButton> ClearButton;

	UClass* TagClass;
};
