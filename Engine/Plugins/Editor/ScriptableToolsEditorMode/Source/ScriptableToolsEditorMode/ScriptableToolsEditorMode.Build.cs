// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ScriptableToolsEditorMode : ModuleRules
{
	public ScriptableToolsEditorMode(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);


		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);


		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Slate",
				"SlateCore",
				"Engine",
				"InputCore",
				"UnrealEd",
				"EditorFramework",
				"ToolWidgets",
				"EditorWidgets",
				"DeveloperSettings",
				"PropertyEditor",
				"StatusBar",
				"Projects",

				"Core",
				"PhysicsCore",
				"RenderCore",

				"GeometryCore",
				"InteractiveToolsFramework",
				"EditorInteractiveToolsFramework",
				"ScriptableToolsFramework",
				"EditorScriptableToolsFramework",
				"ModelingComponentsEditorOnly",

				"ModelingComponents",

				"WidgetRegistration"


			}
			);


		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Engine",
				"AssetRegistry",
				"KismetWidgets"

			}
			);
		
		if (Target.bBuildDeveloperTools)
		{
			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ToolWidgets"
			}
			);
		}

		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);

	}
}
